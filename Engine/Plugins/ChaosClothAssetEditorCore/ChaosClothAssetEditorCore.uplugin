{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Cloth Asset Editor Core",
	"Description": "Core required functionalities for editing and creating Dataflow based Cloth Assets.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosClothAssetEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		},
		{
			"Name": "ChaosClothAssetEditorTools",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "ChaosCaching",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetDataflowNodes",
			"Enabled": true
		},
		{
			"Name": "ChaosCloth",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "BaseCharacterFXEditor",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "MeshResizing",
			"Enabled": true
		}
	]
}
