[CoreRedirects]
+PackageRedirects=(OldName="/ChaosClothAssetEditor/BP_ClothPreview", NewName="/ChaosClothAssetEditorCore/BP_ClothPreview")
