// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetEditor : ModuleRules
{
	public ChaosClothAssetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				// ... add other public dependencies that you statically link with here ...
				
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"BaseCharacterFXEditor",
				"Core",
				"UnrealEd",
				"InputCore",
				"InteractiveToolsFramework",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"Projects",
				"AdvancedPreviewScene",
				"ContentBrowser",
				"DynamicMesh",
				"EditorFramework", // FEditorModeInfo
				"EditorInteractiveToolsFramework",
				"ModelingComponentsEditorOnly", // Static/skeletal mesh tool targets
				"ModelingComponents",
				"MeshConversion",
				"MeshDescription",
				"MeshModelingToolsExp",
				"MeshModelingToolsEditorOnlyExp",
				"EditorStyle",
				"EditorSubsystem",
				"GeometryCore",
				"GeometryFramework",
				"LevelEditor",
				"StatusBar",
				"ToolMenus",
				"ToolWidgets",
				"WorkspaceMenuStructure",
				"ClothingSystemRuntimeCommon",
				"ClothingSystemEditorInterface",
				"ClothingSystemRuntimeInterface",
				"Chaos",
				"ChaosCloth",
				"ChaosClothAsset",
				"ChaosClothAssetEngine",
				"ChaosClothAssetDataflowNodes",
				"ClothingSystemEditor",
				"ChaosClothAssetTools",
				"ChaosClothAssetEditorTools",
				"DataflowCore",
				"DataflowEditor",
				"DataflowEngine",
				"DataflowEnginePlugin",
				"RenderCore",
				"TypedElementFramework",
				"TypedElementRuntime",
				"AssetDefinition",
				"EditorWidgets",
				"KismetWidgets",      // SScrubControlPanel
				"AnimGraph",          // UAnimSingleNodeInstance
				"DeveloperSettings"
				// ... add private dependencies that you statically link with here ...	
			}
		);

		PrivateIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "Internal"));  // For thumbnail renderer
		PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetDataflowNodes"), "Internal"));  // For ReferenceBoneNode
	}
}
