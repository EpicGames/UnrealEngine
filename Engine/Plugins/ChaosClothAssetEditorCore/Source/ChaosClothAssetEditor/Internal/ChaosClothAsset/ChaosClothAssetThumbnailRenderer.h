// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"

#include "ChaosClothAssetThumbnailRenderer.generated.h"

class UChaosClothComponent;
class UChaosClothAssetBase;

/**
 * Minimal actor with a cloth component used to render cloth assets previews
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") AChaosClothPreviewActor : public AActor
{
	GENERATED_BODY()
public:
	AChaosClothPreviewActor();

	UChaosClothComponent* GetClothComponent()
	{
		return ClothComponent;
	}

protected:
	UPROPERTY()
	TObjectPtr<UChaosClothComponent> ClothComponent;
};

namespace UE::Chaos::ClothAsset
{
	/**
	 * Preview scene for a Cloth asset thumbnail
	 */
	class UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") FThumbnailScene : public FThumbnailPreviewScene
	{
	public:
		FThumbnailScene();

		void SetClothAsset(UChaosClothAssetBase* InClothAsset);

		virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		AChaosClothPreviewActor* PreviewActor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
}


/**
 * Renders a preview scene to be used as a thumbnail for a supported cloth asset
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") UChaosClothAssetThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;

	virtual void BeginDestroy() override;

protected:
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
	TObjectInstanceThumbnailScene<UE::Chaos::ClothAsset::FThumbnailScene, 128> ClothThumbnailSceneCache;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
