// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ClothAsset.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothEditor.h"
#include "ChaosClothAsset/ColorScheme.h"
#include "Dataflow/DataflowObject.h"
#include "Misc/MessageDialog.h"
#include "Misc/FileHelper.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Misc/WarnIfAssetsLoadedInScope.h"
#include "Dialog/SMessageDialog.h"
#include "ChaosClothAsset/TerminalNode.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Toolkits/SimpleAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ClothAsset)

#define LOCTEXT_NAMESPACE "AssetDefinition_ClothAsset"

namespace UE::Chaos::ClothAsset::Private
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static bool bUseClothPanelEditorByDefault = false;
	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static FAutoConsoleVariableRef CVarUseClothPanelEditorByDefault(
		TEXT("p.ChaosCloth.UseClothPanelEditorByDefault"),
		bUseClothPanelEditorByDefault,
		TEXT("Enable the use of the deprecated Cloth Panel Editor instead of the unified Dataflow one for editing Cloth Assets (see p.ChaosCloth.AllowClothPanelEditor)."));

	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static bool bAllowClothPanelEditor = false;
	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static FAutoConsoleVariableRef CVarAllowClothPanelEditor(
		TEXT("p.ChaosCloth.AllowClothPanelEditor"),
		bAllowClothPanelEditor,
		TEXT("Allow the use of the deprecated Cloth Panel Editor for editing Cloth Assets."));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FText UAssetDefinition_ClothAsset::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ClothAsset", "ClothAsset");
}

TSoftClassPtr<UObject> UAssetDefinition_ClothAsset::GetAssetClass() const
{
	return UChaosClothAsset::StaticClass();
}

FLinearColor UAssetDefinition_ClothAsset::GetAssetColor() const
{
	return UE::Chaos::ClothAsset::FColorScheme::Asset;
}

bool UAssetDefinition_ClothAsset::CanImport() const
{
	return true;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ClothAsset::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, LOCTEXT("AssetDefinition_ClothAssetSubMenu", "Chaos Cloth"), ECategoryMenuType::Section)
		};
	return Categories;
}

UThumbnailInfo* UAssetDefinition_ClothAsset::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_ClothAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UChaosClothAsset*> ClothObjects = OpenArgs.LoadObjects<UChaosClothAsset>();

	// For now the cloth editor only works on one asset at a time
	ensure(ClothObjects.Num() == 0 || ClothObjects.Num() == 1);
	if (ClothObjects.Num() > 0)
	{
		if (ClothObjects[0])
		{
			bool bSuccess = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
			if (AllowClothPanelEditor() && UseClothPanelEditorByDefault())
			{
				bSuccess = UAssetDefinition_ClothAsset::LaunchClothPanelAssetEditor(ClothObjects[0]);
			}
			else
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				bSuccess = UAssetDefinition_ClothAsset::LaunchClothDataflowAssetEditor(ClothObjects[0]);
			}
			if (!bSuccess)
			{
				// fallback to basic property panel editor
				FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, ClothObjects[0]);
			}

			return EAssetCommandResult::Handled;
		}
	}
	return EAssetCommandResult::Unhandled;
}

bool UAssetDefinition_ClothAsset::UseClothPanelEditorByDefault()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	return UE::Chaos::ClothAsset::Private::bUseClothPanelEditorByDefault;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetDefinition_ClothAsset::AllowClothPanelEditor()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	return UE::Chaos::ClothAsset::Private::bAllowClothPanelEditor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetDefinition_ClothAsset::LaunchClothPanelAssetEditor(UChaosClothAsset* InClothAsset)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	if (ensure(AllowClothPanelEditor()) && InClothAsset)
	{
		if (UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			if (UChaosClothAssetEditor* const AssetEditor = NewObject<UChaosClothAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient))
			{
				AssetEditor->Initialize({ InClothAsset });
				return true;
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return false;
}

bool UAssetDefinition_ClothAsset::LaunchClothDataflowAssetEditor(UChaosClothAsset* InClothAsset)
{
	if (InClothAsset)
	{
		// Check if the cloth asset has a Dataflow asset
		if (InClothAsset->GetDataflow())
		{
			if (UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				if (UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient))
				{
					if (UDataflowSimulationSettings* const SimulationSettings = NewObject<UDataflowSimulationSettings>())
					{
						SimulationSettings->bIsSimulationPlayingByDefault = true;
						SimulationSettings->bIsAsyncCachingSupported = false;
						SimulationSettings->bIsAsyncCachingEnabledByDefault = false;
						AssetEditor->AddEditorSettings(SimulationSettings);
					}

					if (UDataflowEvaluationSettings* const EvaluationSettings = NewObject<UDataflowEvaluationSettings>())
					{
						EvaluationSettings->bAllowEvaluationInPIE = true;
						AssetEditor->AddEditorSettings(EvaluationSettings);
					}

					const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr,
						TEXT("/ChaosClothAssetEditor/BP_ClothPreview.BP_ClothPreview_C"), nullptr, LOAD_None, nullptr);
					AssetEditor->RegisterToolCategories({ "General", "Cloth" });
					AssetEditor->Initialize({ InClothAsset }, ActorClass);

					return true;
				}
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
