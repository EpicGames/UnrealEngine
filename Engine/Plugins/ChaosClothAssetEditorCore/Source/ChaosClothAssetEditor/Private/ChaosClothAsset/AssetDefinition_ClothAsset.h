// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_ClothAsset.generated.h"

class UChaosClothAsset;

UCLASS()
class UAssetDefinition_ClothAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	static bool LaunchClothDataflowAssetEditor(UChaosClothAsset* ClothAsset);

	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static bool LaunchClothPanelAssetEditor(UChaosClothAsset* ClothAsset);

	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static bool UseClothPanelEditorByDefault();

	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	static bool AllowClothPanelEditor();

private:
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual bool CanImport() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

