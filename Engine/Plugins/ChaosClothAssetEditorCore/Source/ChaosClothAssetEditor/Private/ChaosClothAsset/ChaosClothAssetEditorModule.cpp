// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ChaosClothAssetEditorModule.h"


#include "ChaosClothAsset/AssetDefinition_ClothAsset.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothDataflowConstructionVisualization.h"
#include "ChaosClothAsset/ClothDataflowSimulationVisualization.h"
#include "ChaosClothAsset/ClothEditor.h"
#include "ChaosClothAsset/ClothSimulationNodeDetailExtender.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorOptions.h"
#include "ChaosClothAsset/ClothEditorStyle.h"
#include "ChaosClothAsset/LegacyClothingConverter.h"
#include "ChaosClothAsset/LegacyClothingConverterProvider.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "Algo/AllOf.h"
#include "ContentBrowserMenuContexts.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowEditorModeUILayer.h"
#include "Dataflow/DataflowEditor.h"
#include "ComponentAssetBroker.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetEditorModule"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Concrete ILegacyClothingConverterProvider implementation that lets ChaosClothAssetTools
	 * (which sits below us in the dependency graph) drive the new Dataflow-graph-based converter
	 * via IModularFeatures. Registered in StartupModule, unregistered in ShutdownModule.
	 */
	class FLegacyClothingConverterProvider final : public ILegacyClothingConverterProvider
	{
	public:
		virtual bool ConvertInto(const UClothingAssetCommon* Source, UChaosClothAsset* Target) const override
		{
			const FLegacyClothingConverterResult Result = FLegacyClothingConverter::ConvertInto(Source, Target);
			return Result.CreatedAsset != nullptr;
		}
	};

	struct FClothAssetComponentBroker : public IComponentAssetBroker
	{
	public:
		UClass* GetSupportedAssetClass() override
		{
			return UChaosClothAsset::StaticClass();
		}

		virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
		{
			if (UChaosClothComponent* ClothComponent = Cast<UChaosClothComponent>(InComponent))
			{
				if (UChaosClothAsset* ClothAsset = Cast<UChaosClothAsset>(InAsset))
				{
					ClothComponent->SetAsset(ClothAsset);
					return true;
				}
			}
			return false;
		}

		virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
		{
			if (UChaosClothComponent* ClothComponent = Cast<UChaosClothComponent>(InComponent))
			{
				return Cast<UChaosClothAsset>(ClothComponent->GetAsset());
			}
			return nullptr;
		}
	};

	void FChaosClothAssetEditorModule::RegisterMenus()
	{
		// Allows cleanup when module unloads.
		FToolMenuOwnerScoped OwnerScoped(this);

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
		if (UAssetDefinition_ClothAsset::AllowClothPanelEditor())
		{
			// Enable opening ChaosClothAssets in the Dataflow Editor via the Content Browser context menu
			UToolMenu* const ClothContextMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.ChaosClothAsset");
			FToolMenuSection& Section = ClothContextMenu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry("OpenInAlternateEditor", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				// We'll need to get the target assets out of the context
				if (const UContentBrowserAssetContextMenuContext* const Context = Section.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					// We are deliberately not using Context->GetSelectedObjects() here to avoid triggering a load from right clicking
					// an asset in the content browser.
					const bool bAllSelectedAssetsAreCloth = Algo::AllOf(Context->SelectedAssets, [](const FAssetData& Asset)
					{
						return Asset.AssetClassPath == UChaosClothAsset::StaticClass()->GetClassPathName();
					});

					if (bAllSelectedAssetsAreCloth)
					{
						UDataflowEditorUISubsystem* const DataflowEditorSubsystem = GEditor->GetEditorSubsystem<UDataflowEditorUISubsystem>();
						check(DataflowEditorSubsystem);

						const TSharedPtr<class FUICommandList> CommandListToBind = MakeShared<FUICommandList>();

						const bool bLaunchDataflowEditorAsAlternative = UAssetDefinition_ClothAsset::UseClothPanelEditorByDefault();

						TSharedPtr<FUICommandInfo> LaunchCommand =
							bLaunchDataflowEditorAsAlternative
							? FChaosClothAssetEditorCommands::Get().OpenClothAssetInDataflowEditor
							: FChaosClothAssetEditorCommands::Get().OpenClothAssetInClothPanelEditor
							;

						CommandListToBind->MapAction(
							LaunchCommand,
							FExecuteAction::CreateWeakLambda(DataflowEditorSubsystem, 
								[Context, DataflowEditorSubsystem, bLaunchDataflowEditorAsAlternative]()
								{
									// When we actually do want to open the editor, trigger the load to get the objects
									TArray<TObjectPtr<UObject>> AssetsToEdit;
									AssetsToEdit.Append(Context->LoadSelectedObjects<UObject>());
									if (AssetsToEdit.Num() > 0)
									{
										if (UChaosClothAsset* const ClothAsset = CastChecked<UChaosClothAsset>(AssetsToEdit[0]))
										{
											if (bLaunchDataflowEditorAsAlternative)
											{
												UAssetDefinition_ClothAsset::LaunchClothDataflowAssetEditor(ClothAsset);
											}
											else
											{
												UAssetDefinition_ClothAsset::LaunchClothPanelAssetEditor(ClothAsset);
											}
										}
									}
								}),
							FCanExecuteAction::CreateWeakLambda(Context, [Context]() { return Context->bCanBeModified; }));

						const TAttribute<FText> ToolTipOverride = Context->bCanBeModified ? TAttribute<FText>() : LOCTEXT("ReadOnlyAssetWarning", "The selected asset(s) are read-only and cannot be edited.");
						Section.AddMenuEntryWithCommandList(
							LaunchCommand,
							CommandListToBind,
							TAttribute<FText>(),
							ToolTipOverride,
							FSlateIcon());		// TODO: If DataflowEditorStyle.h was public we could do this: FSlateIcon(FDataflowEditorStyle::Get().GetStyleSetName(), "ClassThumbnail.Dataflow")
					}
				}
			}));
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FChaosClothAssetEditorModule::StartupModule()
	{
		FChaosClothAssetEditorStyle::Get(); // Causes the constructor to be called

		FChaosClothAssetEditorCommands::Register();

		// Menus need to be registered in a callback to make sure the system is ready for them.
		StartupCallbackDelegateHandle = UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FChaosClothAssetEditorModule::RegisterMenus));

		Dataflow::FDataflowConstructionVisualizationRegistry::GetInstance().RegisterVisualization(MakeUnique<FClothDataflowConstructionVisualization>());
		Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().RegisterVisualization(MakeUnique<FClothDataflowSimulationVisualization>());
		Dataflow::FDataflowNodeDetailExtensionRegistry::GetInstance().RegisterExtension(MakeUnique<FClothSimulationNodeDetailExtender>());

		if (IConsoleVariable* const Var = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ChaosCloth.EnableDataflowEditor")))	// declared in AssetDefinition_ClothAsset.cpp
		{
			if (UChaosClothEditorOptions* const Options = UChaosClothEditorOptions::StaticClass()->GetDefaultObject<UChaosClothEditorOptions>())
			{
				Var->Set(Options->bClothAssetsOpenInDataflowEditor);

				OnCVarChangedDelegateHandle = Var->OnChangedDelegate().AddLambda([Options](IConsoleVariable* Variable)
				{
					Options->bClothAssetsOpenInDataflowEditor = Variable->GetBool();
					Options->SaveConfig();
				});
			}
		}

		ClothAssetComponentBroker = MakeShareable(new FClothAssetComponentBroker);
		FComponentAssetBrokerage::RegisterBroker(ClothAssetComponentBroker, UChaosClothComponent::StaticClass(), true, true);

		DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UChaosClothAsset::StaticClass());

		// Register the legacy -> ChaosClothAsset converter as a modular feature so the right-click
		// "Export to Chaos Cloth Asset" exporter in ChaosClothAssetTools (which sits below this
		// module in the dependency graph) can call into the new Dataflow-graph-based converter.
		LegacyClothingConverterProvider = MakeUnique<FLegacyClothingConverterProvider>();
		IModularFeatures::Get().RegisterModularFeature(
			ILegacyClothingConverterProvider::FeatureName,
			LegacyClothingConverterProvider.Get());
	}

	void FChaosClothAssetEditorModule::ShutdownModule()
	{
		// Unregister the legacy conversion provider so the exporter falls back to its in-tree
		// flat-mesh path if it is invoked after we unload.
		if (LegacyClothingConverterProvider)
		{
			IModularFeatures::Get().UnregisterModularFeature(
				ILegacyClothingConverterProvider::FeatureName,
				LegacyClothingConverterProvider.Get());
			LegacyClothingConverterProvider.Reset();
		}

		if (UObjectInitialized())
		{
			FComponentAssetBrokerage::UnregisterBroker(ClothAssetComponentBroker);
		}

		if (IConsoleVariable* const Var = IConsoleManager::Get().FindConsoleVariable(TEXT("p.ChaosCloth.EnableDataflowEditor")))
		{
			Var->OnChangedDelegate().Remove(OnCVarChangedDelegateHandle);
		}

		UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);

		Dataflow::FDataflowNodeDetailExtensionRegistry::GetInstance().DeregisterExtension(FClothSimulationNodeDetailExtender::Name);
		Dataflow::FDataflowConstructionVisualizationRegistry::GetInstance().DeregisterVisualization(FClothDataflowConstructionVisualization::Name);
		Dataflow::FDataflowSimulationVisualizationRegistry::GetInstance().DeregisterVisualization(FClothDataflowSimulationVisualization::Name);

		UToolMenus::UnRegisterStartupCallback(StartupCallbackDelegateHandle);

		FChaosClothAssetEditorCommands::Unregister();

		FEditorModeRegistry::Get().UnregisterMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId);
	}

} // namespace UE::Chaos::ClothAsset

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FChaosClothAssetEditorModule, ChaosClothAssetEditor)

#undef LOCTEXT_NAMESPACE
