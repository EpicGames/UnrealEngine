// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModule.h"

namespace UE::Chaos::ClothAsset
{
	struct FClothAssetComponentBroker;
	class FLegacyClothingConverterProvider;

	class FChaosClothAssetEditorModule final : public FBaseCharacterFXEditorModule
	{
	public:
		/** IModuleInterface implementation */
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:

		FDelegateHandle StartupCallbackDelegateHandle;
		FDelegateHandle OnCVarChangedDelegateHandle;
		FDelegateHandle DataflowMenusHandle;

		void RegisterMenus();

		TSharedPtr<FClothAssetComponentBroker> ClothAssetComponentBroker;
		TUniquePtr<FLegacyClothingConverterProvider> LegacyClothingConverterProvider;
	};
} // namespace UE::Chaos::ClothAsset
