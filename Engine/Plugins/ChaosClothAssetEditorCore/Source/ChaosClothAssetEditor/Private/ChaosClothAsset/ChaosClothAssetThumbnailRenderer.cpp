// Copyright Epic Games, Inc. All Rights Reserved.

// UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.")
// TODO: Remove these files in 5.10.

#include "ChaosClothAsset/ChaosClothAssetThumbnailRenderer.h"

#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "SceneView.h"
#include "ShowFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosClothAssetThumbnailRenderer)

AChaosClothPreviewActor::AChaosClothPreviewActor()
{
	ClothComponent = CreateDefaultSubobject<UChaosClothComponent>(TEXT("ClothComponent0"));
	RootComponent = ClothComponent;
}

namespace UE::Chaos::ClothAsset
{
	FThumbnailScene::FThumbnailScene() : FThumbnailPreviewScene()
	{
		bForceAllUsedMipsResident = false;

		FActorSpawnParameters SpawnInfo;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.bNoFail = true;
		SpawnInfo.ObjectFlags = RF_Transient;

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		PreviewActor = GetWorld()->SpawnActor<AChaosClothPreviewActor>( SpawnInfo );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		PreviewActor->SetActorEnableCollision(false);

		check(PreviewActor);
	}

	void FThumbnailScene::SetClothAsset(UChaosClothAssetBase* InClothAsset)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		UChaosClothComponent* const ClothComponent = PreviewActor->GetClothComponent();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		check(ClothComponent);
		
		ClothComponent->SetAsset(InClothAsset);
	}

	void FThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw,
		float& OutOrbitZoom) const
	{	
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		UChaosClothComponent* const ClothComponent = PreviewActor->GetClothComponent();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		check(ClothComponent);

		FBoxSphereBounds Bounds = ClothComponent->Bounds;
		Bounds = Bounds.ExpandBy(2.0f);
		
		const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
		const float HalfMeshSize = static_cast<float>(Bounds.SphereRadius); 
		const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

		USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(ClothComponent->GetThumbnailInfo());
		if (ThumbnailInfo)
		{
			if (TargetDistance + ThumbnailInfo->OrbitZoom < 0 )
			{
				ThumbnailInfo->OrbitZoom = -TargetDistance;
			}
		}
		else
		{
			ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
		}

		OutOrigin = -Bounds.Origin;
		OutOrbitPitch = ThumbnailInfo->OrbitPitch;
		OutOrbitYaw = ThumbnailInfo->OrbitYaw;
		OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
	}
}

void UChaosClothAssetThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UChaosClothAssetBase* const ClothAsset = Cast<UChaosClothAssetBase>(Object);
	if (!ClothAsset)
	{
		return;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
	TSharedRef<UE::Chaos::ClothAsset::FThumbnailScene> ThumbnailScene = ClothThumbnailSceneCache.EnsureThumbnailScene(Object);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Check if the cloth asset has render data, if not render the sim mesh instead
	FSkeletalMeshRenderData* const RenderData = ClothAsset->GetResourceForRendering();
	const bool bHasRenderData = RenderData && RenderData->LODRenderData.Num() && RenderData->LODRenderData[0].GetTotalFaces();

	if (!bHasRenderData && ClothAsset->HasValidClothSimulationModels() && ClothAsset->GetCollections(0).Num())
	{
		// Render sim mesh
		const FSoftObjectPath RenderMaterialPathName(TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"));

		const TSharedRef<const FManagedArrayCollection>& Collection = ClothAsset->GetCollections(0)[0];
		UChaosClothAsset* const ClothAssetCopy = NewObject<UChaosClothAsset>(this, UChaosClothAsset::StaticClass());

		TSharedRef<FManagedArrayCollection> CollectionCopy = MakeShared<FManagedArrayCollection>(*Collection);

		// Reverse mesh normals to keep the same render look as inverted sim mesh in flat shaded visualization
		constexpr bool bReverseSimMeshNormals = true;
		constexpr bool bReverseSimMeshWindingOrder = false;
		constexpr bool bReverseRenderMeshNormals = false;
		constexpr bool bReverseRenderMeshWindingOrder = false;
		const TArray<int32> EmptySelection;

		UE::Chaos::ClothAsset::FClothGeometryTools::ReverseMesh(
			CollectionCopy,
			bReverseSimMeshNormals,
			bReverseSimMeshWindingOrder,
			bReverseRenderMeshNormals,
			bReverseRenderMeshWindingOrder,
			EmptySelection,
			EmptySelection);

		constexpr bool bSingleRenderPattern = true;
		UE::Chaos::ClothAsset::FClothGeometryTools::CopySimMeshToRenderMesh(CollectionCopy, RenderMaterialPathName, bSingleRenderPattern);

		const TArray<TSharedRef<const FManagedArrayCollection>> ClothCollectionsCopy = { MoveTemp(CollectionCopy) };

		ClothAssetCopy->Build(ClothCollectionsCopy);
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		ThumbnailScene->SetClothAsset(ClothAssetCopy);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		// Render render mesh
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
		ThumbnailScene->SetClothAsset(ClothAsset);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( Viewport, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
		.SetTime(UThumbnailRenderer::GetTime())
		.SetAdditionalViewFamily(bAdditionalViewFamily));

	ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	ViewFamily.EngineShowFlags.MotionBlur = 0;
	ViewFamily.EngineShowFlags.LOD = 0;

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "This class will be moved to the internal section of the ChaosClothAsset module.") 
	RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	ThumbnailScene->SetClothAsset(nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UChaosClothAssetThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Cast<UChaosClothAssetBase>(Object) != nullptr;
}

EThumbnailRenderFrequency UChaosClothAssetThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	UChaosClothAssetBase* AsClothAsset = Cast<UChaosClothAssetBase>(Object);
	return AsClothAsset && AsClothAsset->GetResourceForRendering() ? EThumbnailRenderFrequency::Realtime : EThumbnailRenderFrequency::OnPropertyChange;
}

void UChaosClothAssetThumbnailRenderer::BeginDestroy()
{
	ClothThumbnailSceneCache.Clear();
	Super::BeginDestroy();
}
