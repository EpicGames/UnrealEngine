// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothDataflowConstructionVisualization.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowConstructionScene.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowConstructionViewportClient.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshBuilder.h"
#include "SceneView.h"
#include "Materials/Material.h"
#include "EditorModes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetDataflowConstructionVisualization"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		extern FLinearColor PseudoRandomColor(int32 NumColorRotations);

		void RenderSeams(const UE::Chaos::ClothAsset::FCollectionClothConstFacade& ClothFacade,
			const UE::Geometry::FDynamicMesh3* const Mesh,
			const UE::Dataflow::IDataflowConstructionViewMode* const ViewMode,
			const TSharedRef<const FManagedArrayCollection>& ClothCollection,
			bool bCollapseSeams,
			FPrimitiveDrawInterface* PDI)
		{
			const bool bIs2DSim = ViewMode->GetName() == FName("Cloth2DSimView");
			const bool bIs3DSim = ViewMode->GetName() == FName("Cloth3DSimView");

			if (!bIs2DSim && !bIs3DSim)
			{
				return;
			}

			const float LineThickness = bIs3DSim ? 4.0f : 2.0f;
			constexpr float PointSize = 4.0f;

			int32 ConnectedSeamIndex = 0;		// Used to generate different colors for each connected seam, if multiple connected seams are found per input seam

			for (int32 SeamIndex = 0; SeamIndex < ClothFacade.GetNumSeams(); ++SeamIndex)
			{
				const UE::Chaos::ClothAsset::FCollectionClothSeamConstFacade SeamFacade = ClothFacade.GetSeam(SeamIndex);

				if (bIs2DSim)
				{
					// Stitches are given in random order, so first construct paths of connected stitches
					// Note one SeamFacade can contain multiple disjoint paths
					TArray<TArray<FIntVector2>> ConnectedSeams;
					UE::Chaos::ClothAsset::FClothGeometryTools::BuildConnectedSeams2D(ClothCollection, SeamIndex, *Mesh, ConnectedSeams);

					for (const TArray<FIntVector2>& ConnectedSeam : ConnectedSeams)
					{
						const FColor SeamColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(ConnectedSeamIndex++).ToFColor(true);

						// draw connected edge on each side of the seam
						for (int32 StitchID = 0; StitchID < ConnectedSeam.Num() - 1; ++StitchID)
						{
							const FVector3d Stitch0Point0 = Mesh->GetVertex(ConnectedSeam[StitchID][0]);
							const FVector3d Stitch0Point1 = Mesh->GetVertex(ConnectedSeam[StitchID][1]);
							const FVector3d Stitch1Point0 = Mesh->GetVertex(ConnectedSeam[StitchID + 1][0]);
							const FVector3d Stitch1Point1 = Mesh->GetVertex(ConnectedSeam[StitchID + 1][1]);

							PDI->DrawLine(Stitch0Point0, Stitch1Point0, SeamColor, SDPG_World);
							PDI->DrawLine(Stitch0Point1, Stitch1Point1, SeamColor, SDPG_World);
						}

						// draw connection between stitch points

						if (bCollapseSeams)
						{
							const int32 StitchID = ConnectedSeam.Num() / 2;
							const int32 StitchVertex0 = ConnectedSeam[StitchID][0];
							const int32 StitchVertex1 = ConnectedSeam[StitchID][1];
							const FVector StitchPoint0 = Mesh->GetVertex(StitchVertex0);
							const FVector StitchPoint1 = Mesh->GetVertex(StitchVertex1);

							PDI->DrawLine(StitchPoint0, StitchPoint1, SeamColor, SDPG_World);
							PDI->DrawPoint(StitchPoint0, SeamColor, PointSize, SDPG_World);
							PDI->DrawPoint(StitchPoint1, SeamColor, PointSize, SDPG_World);
						}
						else
						{
							for (int32 StitchID = 0; StitchID < ConnectedSeam.Num(); ++StitchID)
							{
								const int32 StitchVertex0 = ConnectedSeam[StitchID][0];
								const int32 StitchVertex1 = ConnectedSeam[StitchID][1];
								const FVector StitchPoint0 = Mesh->GetVertex(StitchVertex0);
								const FVector StitchPoint1 = Mesh->GetVertex(StitchVertex1);

								PDI->DrawLine(StitchPoint0, StitchPoint1, SeamColor, SDPG_World);
								PDI->DrawPoint(StitchPoint0, SeamColor, PointSize, SDPG_World);
								PDI->DrawPoint(StitchPoint1, SeamColor, PointSize, SDPG_World);
							}
						}
					}
				}
				else if (bIs3DSim)
				{
					const TArray<int32> SeamStitches(SeamFacade.GetSeamStitch3DIndex());
					const FColor SeamColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(SeamIndex).ToFColor(true);

					// In 3D we should be able to draw the seam edges in any order, doesn't need to be in connected paths
					for (int32 StitchIndexI = 0; StitchIndexI < SeamStitches.Num(); ++StitchIndexI)
					{
						const int32 StitchIVertex = SeamStitches[StitchIndexI];
						for (int32 StitchIndexJ = StitchIndexI + 1; StitchIndexJ < SeamStitches.Num(); ++StitchIndexJ)
						{
							const int32 StitchJVertex = SeamStitches[StitchIndexJ];

							if (Mesh->FindEdge(StitchIVertex, StitchJVertex) != UE::Geometry::FDynamicMesh3::InvalidID)
							{
								const FVector StitchIPoint = Mesh->GetVertex(StitchIVertex);
								const FVector StitchJPoint = Mesh->GetVertex(StitchJVertex);
								PDI->DrawLine(StitchIPoint, StitchJPoint, SeamColor, SDPG_World);
								PDI->DrawPoint(StitchIPoint, SeamColor, PointSize, SDPG_World);
								PDI->DrawPoint(StitchJPoint, SeamColor, PointSize, SDPG_World);
							}
						}
					}
				}
			}
		}

		void RenderPatterns(const UE::Chaos::ClothAsset::FCollectionClothConstFacade& ClothFacade, const UE::Geometry::FDynamicMesh3* const Mesh, FPrimitiveDrawInterface* PDI)
		{

			for (int32 PatternIndex = 0; PatternIndex < ClothFacade.GetNumSimPatterns(); ++PatternIndex)
			{
				const FCollectionClothSimPatternConstFacade Pattern = ClothFacade.GetSimPattern(PatternIndex);
				const FLinearColor LinearPatternColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(PatternIndex);
				const FColor PatternColor = LinearPatternColor.ToFColorSRGB();

				FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

				for (int32 PatternFaceID = 0; PatternFaceID < Pattern.GetNumSimFaces(); ++PatternFaceID)
				{
					const int32 MeshTriangleID = Pattern.GetSimFacesOffset() + PatternFaceID;
					const FVector3f TrianglePoint0 = (FVector3f)Mesh->GetTriVertex(MeshTriangleID, 0);
					const FVector3f TrianglePoint1 = (FVector3f)Mesh->GetTriVertex(MeshTriangleID, 1);
					const FVector3f TrianglePoint2 = (FVector3f)Mesh->GetTriVertex(MeshTriangleID, 2);

					const int32 VertexIndex0 = MeshBuilder.AddVertex(FDynamicMeshVertex(TrianglePoint0));
					const int32 VertexIndex1 = MeshBuilder.AddVertex(FDynamicMeshVertex(TrianglePoint1));
					const int32 VertexIndex2 = MeshBuilder.AddVertex(FDynamicMeshVertex(TrianglePoint2));
					MeshBuilder.AddTriangle(VertexIndex0, VertexIndex1, VertexIndex2);
				}

				FDynamicColoredMaterialRenderProxy* PatternColorMaterial = new FDynamicColoredMaterialRenderProxy(GEngine->EmissiveMeshMaterial->GetRenderProxy(), UE::Chaos::ClothAsset::Private::PseudoRandomColor(PatternIndex));
				PDI->RegisterDynamicResource(PatternColorMaterial);

				MeshBuilder.Draw(PDI, FMatrix::Identity, PatternColorMaterial, SDPG_World, false, false);
			}
		}
	}

	const FName FClothDataflowConstructionVisualization::Name = FName("ClothDataflowConstructionVisualization");

	FName FClothDataflowConstructionVisualization::GetName() const
	{
		return Name;
	}

	void FClothDataflowConstructionVisualization::ExtendViewportShowMenu(const TSharedPtr<FDataflowConstructionViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder)
	{
		TSharedPtr<const FManagedArrayCollection> Collection;

		if (const FDataflowConstructionScene* ConstructionScene = StaticCast<FDataflowConstructionScene*>(ViewportClient->GetPreviewScene()))
		{
			if (const TObjectPtr<UDataflowBaseContent> DataflowContent = ConstructionScene->GetEditorContent())
			{
				Collection = DataflowContent->GetSelectedCollection();
			}
		}

		if (!Collection)
		{
			return;
		}

		const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection.ToSharedRef());
		if (!ClothFacade.IsValid())
		{
			return;
		}

		MenuBuilder.BeginSection(TEXT("ClothSeamVisualization"), LOCTEXT("ClothSeamVisualizationSectionName", "Chaos Cloth"));
		{
			const FUIAction SeamToggleAction(
				FExecuteAction::CreateLambda([this, ViewportClient]()
				{
					bSeamVisualizationEnabled = !bSeamVisualizationEnabled;
					if (ViewportClient)
					{
						ViewportClient->Invalidate();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					return bSeamVisualizationEnabled;
				}));

			MenuBuilder.AddMenuEntry(LOCTEXT("ClothSeamVisualizationOptionButtonText", "Seams"),
				LOCTEXT("ClothSeamVisualizationOptionTooltipText", "Seams visualization"),
				FSlateIcon(),
				SeamToggleAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton);

			const FUIAction SeamCollapseToggleAction(
				FExecuteAction::CreateLambda([this, ViewportClient]()
				{
					bCollapseSeams = !bCollapseSeams;
					if (ViewportClient)
					{
						ViewportClient->Invalidate();
					}
				}),
				FCanExecuteAction::CreateLambda([this, ViewportClient]()
				{
					return bSeamVisualizationEnabled;
				}),
				FIsActionChecked::CreateLambda([this]()
				{
					return bCollapseSeams;
				}));

			MenuBuilder.AddMenuEntry(LOCTEXT("CollapseSeamsVisualizationOptionText", "Collapse Seams"),
				LOCTEXT("CollapseSeamsVisualizationOptionTooltipText", "Collapse seams connection in seams visualization"),
				FSlateIcon(),
				SeamCollapseToggleAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton);


			const FUIAction PatternColorToggleAction(
				FExecuteAction::CreateLambda([this, ViewportClient]()
				{
					bPatternColorVisualizationEnabled = !bPatternColorVisualizationEnabled;
					if (ViewportClient)
					{
						ViewportClient->Invalidate();
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]()
				{
					return bPatternColorVisualizationEnabled;
				}));

			MenuBuilder.AddMenuEntry(LOCTEXT("ClothPatternColorVisualizationEnabledOptionText", "Color Patterns"),
				LOCTEXT("ClothPatternColorVisualizationEnabledOptionTooltipText", "Draw each cloth patterns with a different color"),
				FSlateIcon(),
				PatternColorToggleAction,
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}

		MenuBuilder.EndSection();
	}

	void FClothDataflowConstructionVisualization::Draw(const FDataflowConstructionScene* ConstructionScene, FPrimitiveDrawInterface* PDI, const FSceneView* View)
	{
		if (!ConstructionScene)
		{
			return;
		}

		TArray<TObjectPtr<UDynamicMeshComponent>> SceneComponents = ConstructionScene->GetDynamicMeshComponents();
		if (SceneComponents.Num() != 1)
		{
			return;
		}
		const UDynamicMeshComponent* const MeshComponent = SceneComponents[0];
		checkf(MeshComponent, TEXT("Found null mesh component in the ConstructionScene components"));

		const TObjectPtr<UDataflowBaseContent> DataflowContent = ConstructionScene->GetEditorContent();
		checkf(DataflowContent, TEXT("No DataflowContent in the ConstructionScene"));

		const TSharedPtr<const FManagedArrayCollection> Collection = DataflowContent->GetSelectedCollection();

		if (!Collection)
		{
			return;
		}

		const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection.ToSharedRef());

		if (!ClothFacade.IsValid())
		{
			return;
		}

		const UE::Geometry::FDynamicMesh3* Mesh = MeshComponent->GetMesh();
		checkf(Mesh, TEXT("Unexpected null mesh in a DynamicMeshComponent"));

		const UE::Dataflow::IDataflowConstructionViewMode* const ViewMode = DataflowContent->GetConstructionViewMode();
		checkf(ViewMode, TEXT("Found null view mode in DataflowContent"));

		if (bPatternColorVisualizationEnabled)
		{
			Private::RenderPatterns(ClothFacade, Mesh, PDI);
		}

		if (bSeamVisualizationEnabled)
		{
			Private::RenderSeams(ClothFacade, Mesh, ViewMode, Collection.ToSharedRef(), bCollapseSeams, PDI);
		}
		
	}

}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE 
 