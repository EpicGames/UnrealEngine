// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowConstructionVisualization.h"

namespace UE::Chaos::ClothAsset
{
	class FClothDataflowConstructionVisualization : public UE::Dataflow::IDataflowConstructionVisualization
	{
	public:
		static const FName Name;
		virtual ~FClothDataflowConstructionVisualization() = default;

	private:

		// IDataflowConstructionVisualization
		virtual FName GetName() const override;
		virtual void ExtendViewportShowMenu(const TSharedPtr<FDataflowConstructionViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) override;
		virtual void Draw(const FDataflowConstructionScene* ConstructionScene, FPrimitiveDrawInterface* PDI, const FSceneView* View = nullptr) override;

		bool bSeamVisualizationEnabled = false;
		bool bCollapseSeams = false;
		bool bPatternColorVisualizationEnabled = false;
	};

}  // End namespace UE::Chaos::ClothAsset
