// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothDataflowSimulationVisualization.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Dataflow/DataflowSimulationScene.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetDataflowSimulationVisualization"

namespace UE::Chaos::ClothAsset
{
	const FName FClothDataflowSimulationVisualization::Name = FName("ClothDataflowSimulationVisualization");

	FName FClothDataflowSimulationVisualization::GetName() const
	{
		return Name;
	}

	void FClothDataflowSimulationVisualization::ExtendSimulationVisualizationMenu(const TSharedPtr<FDataflowSimulationViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder)
	{
		if (ViewportClient)
		{
			if (TSharedPtr<FDataflowEditorToolkit> Toolkit = ViewportClient->GetDataflowEditorToolkit().Pin())
			{
				if (const TSharedPtr<FDataflowSimulationScene>& SimulationScene = Toolkit->GetSimulationScene())
				{
					if (const UChaosClothComponent* const ClothComponent = GetClothComponent(SimulationScene.Get()))
					{
						ClothEditorSimulationVisualization.RefreshMenusForClothComponent(ClothComponent);  // Update the menus' content before building them
						ClothEditorSimulationVisualization.ExtendViewportShowMenu(MenuBuilder, ViewportClient.ToSharedRef());
					}
				}
			}
		}
	}

	const UChaosClothComponent* FClothDataflowSimulationVisualization::GetClothComponent(const FDataflowSimulationScene* SimulationScene) const
	{
		if (SimulationScene)
		{
			if (const TObjectPtr<AActor> PreviewActor = SimulationScene->GetPreviewActor())
			{
				return PreviewActor->GetComponentByClass<UChaosClothComponent>();
			}
		}
		return nullptr;
	}

	void FClothDataflowSimulationVisualization::Draw(const FDataflowSimulationScene* SimulationScene, FPrimitiveDrawInterface* PDI)
	{
		ClothEditorSimulationVisualization.DebugDrawSimulation(GetClothComponent(SimulationScene), PDI);
	}

	void FClothDataflowSimulationVisualization::DrawCanvas(const FDataflowSimulationScene* SimulationScene, FCanvas* Canvas, const FSceneView* SceneView)
	{
		ClothEditorSimulationVisualization.DebugDrawSimulationTexts(GetClothComponent(SimulationScene), Canvas, SceneView);
	}

	FText FClothDataflowSimulationVisualization::GetDisplayString(const FDataflowSimulationScene* SimulationScene) const
	{
		return ClothEditorSimulationVisualization.GetDisplayString(GetClothComponent(SimulationScene));
	}

}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE 
 