// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothEditorSimulationVisualization.h"
#include "Dataflow/DataflowSimulationVisualization.h"

namespace UE::Chaos::ClothAsset
{
	class FClothDataflowSimulationVisualization : public Dataflow::IDataflowSimulationVisualization
	{
	public:
		static const FName Name;
		virtual ~FClothDataflowSimulationVisualization() = default;

	private:

		// IDataflowSimulationVisualization
		virtual FName GetName() const override;
		virtual void ExtendSimulationVisualizationMenu(const TSharedPtr<FDataflowSimulationViewportClient>& ViewportClient, FMenuBuilder& MenuBuilder) override;
		virtual void Draw(const FDataflowSimulationScene* SimulationScene, FPrimitiveDrawInterface* PDI) override;
		virtual void DrawCanvas(const FDataflowSimulationScene* SimulationScene, FCanvas* Canvas, const FSceneView* SceneView) override;
		virtual FText GetDisplayString(const FDataflowSimulationScene* SimulationScene) const override;

		const UChaosClothComponent* GetClothComponent(const FDataflowSimulationScene* SimulationScene) const;

		FClothEditorSimulationVisualization ClothEditorSimulationVisualization;
	};

}  // End namespace UE::Chaos::ClothAsset
