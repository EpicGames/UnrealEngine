// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditor.h"
#include "ChaosClothAsset/ClothEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditor)

DEFINE_LOG_CATEGORY(LogChaosClothAssetEditor);

TSharedPtr<FBaseAssetToolkit> UChaosClothAssetEditor::CreateToolkit()
{
	using namespace UE::Chaos::ClothAsset;
	TSharedPtr<FChaosClothAssetEditorToolkit> Toolkit = MakeShared<FChaosClothAssetEditorToolkit>(this);
	return Toolkit;
}

