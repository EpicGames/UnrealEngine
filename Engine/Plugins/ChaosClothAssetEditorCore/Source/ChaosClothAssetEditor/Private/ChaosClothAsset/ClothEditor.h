// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditor.h"
#include "ClothEditor.generated.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

/** 
 * The actual asset editor class doesn't have that much in it, intentionally. 
 * 
 * Our current asset editor guidelines ask us to place as little business logic as possible
 * into the class, instead putting as much of the non-UI code into the subsystem as possible,
 * and the UI code into the toolkit (which this class owns).
 *
 * However, since we're using a mode and the Interactive Tools Framework, a lot of our business logic
 * ends up inside the mode and the tools, not the subsystem. The front-facing code is mostly in
 * the asset editor toolkit, though the mode toolkit has most of the things that deal with the toolbar
 * on the left.
 */

UCLASS(MinimalAPI)
class UChaosClothAssetEditor : public UBaseCharacterFXEditor
{
	GENERATED_BODY()

public:

	UE_API virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
};

DECLARE_LOG_CATEGORY_EXTERN(LogChaosClothAssetEditor, Log, All);

#undef UE_API
