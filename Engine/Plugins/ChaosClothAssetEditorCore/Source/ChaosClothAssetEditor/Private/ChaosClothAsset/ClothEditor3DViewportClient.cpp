// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditor3DViewportClient.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorToolkit.h"
#include "ChaosClothAsset/ClothEditorSimulationVisualization.h"
#include "EditorModeManager.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowActor.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "BaseGizmos/TransformProxy.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Animation/SkeletalMeshActor.h"
#include "AssetViewerSettings.h"
#include "ClothEditorCommands.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Transforms/TransformGizmoDataBinder.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"

namespace UE::Chaos::ClothAsset
{
FChaosClothAssetEditor3DViewportClient::FChaosClothAssetEditor3DViewportClient(FEditorModeTools* InModeTools,
	TSharedPtr<FChaosClothPreviewScene> InPreviewScene,
	TSharedPtr<FClothEditorSimulationVisualization> InVisualization,
	const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(InModeTools, InPreviewScene.Get(), InEditorViewportWidget),
	  ClothPreviewScene(InPreviewScene)
	, ClothEditorSimulationVisualization(InVisualization)
{
	// We want our near clip plane to be quite close so that we can zoom in further.
	OverrideNearClipPlane(KINDA_SMALL_NUMBER);

	// Call this once with the default value to get everything in a consistent state
	EnableRenderMeshWireframe(bRenderMeshWireframe);

	EngineShowFlags.SetSelectionOutline(true);

	// Set up Gizmo and TransformProxy

	UModeManagerInteractiveToolsContext* const InteractiveToolsContext = ModeTools->GetInteractiveToolsContext();
	TransformProxy = NewObject<UTransformProxy>();

	UInteractiveGizmoManager* const GizmoManager = InteractiveToolsContext->GizmoManager;
	const FString GizmoIdentifier = TEXT("ChaosClothAssetEditor3DViewportClientGizmoIdentifier");
	Gizmo = GizmoManager->Create3AxisTransformGizmo(this, GizmoIdentifier);

	Gizmo->SetActiveTarget(TransformProxy);
	Gizmo->SetVisibility(false);
	Gizmo->bUseContextGizmoMode = false;
	Gizmo->bUseContextCoordinateSystem = false;
	Gizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;

	UChaosClothPreviewSceneDescription* const SceneDescription = InPreviewScene->GetPreviewSceneDescription();
	DataBinder = MakeShared<FTransformGizmoDataBinder>();
	DataBinder->InitializeBoundVectors(&SceneDescription->Translation, &SceneDescription->Rotation, &SceneDescription->Scale);

	InPreviewScene->SetGizmoDataBinder(DataBinder);

	//
	// Input behaviors
	//

	InputBehaviorSet = NewObject<UInputBehaviorSet>();

	// Our ClickOrDrag behavior is used to intercept non-alt left-mouse-button drag inputs, but still allow single-click for select/deselect operation
	const TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, this);
	
	InputBehaviorSet->Add(ClickOrDragBehavior);

	InteractiveToolsContext->InputRouter->RegisterSource(this);
}

void FChaosClothAssetEditor3DViewportClient::RegisterDelegates()
{
	USelection* const SelectedComponents = ModeTools->GetSelectedComponents();
	SelectedComponents->SelectionChangedEvent.RemoveAll(this);
	SelectedComponents->SelectionChangedEvent.AddSP(this, &FChaosClothAssetEditor3DViewportClient::ComponentSelectionChanged);
}

FChaosClothAssetEditor3DViewportClient::~FChaosClothAssetEditor3DViewportClient()
{
	DeleteViewportGizmo();
	
	if (USelection* const SelectedComponents = ModeTools->GetSelectedComponents())
	{
		SelectedComponents->SelectionChangedEvent.RemoveAll(this);
	}
}

void FChaosClothAssetEditor3DViewportClient::DeleteViewportGizmo()
{
	if (DataBinder && Gizmo && Gizmo->ActiveTarget)
	{
		DataBinder->UnbindFromGizmo(Gizmo, TransformProxy);
	}

	if (Gizmo && ModeTools && ModeTools->GetInteractiveToolsContext() && ModeTools->GetInteractiveToolsContext()->GizmoManager)
	{
		ModeTools->GetInteractiveToolsContext()->GizmoManager->DestroyGizmo(Gizmo);
	}
	Gizmo = nullptr;
	TransformProxy = nullptr;
	DataBinder = nullptr;
}

void FChaosClothAssetEditor3DViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEditorViewportClient::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(TransformProxy);
	Collector.AddReferencedObject(Gizmo);
	Collector.AddReferencedObject(InputBehaviorSet);
}

void FChaosClothAssetEditor3DViewportClient::EnableRenderMeshWireframe(bool bEnable)
{
	bRenderMeshWireframe = bEnable;

	if (UChaosClothComponent* const ClothComponent = GetPreviewClothComponent())
	{
		ClothComponent->SetForceWireframe(bRenderMeshWireframe);
	}
}

void FChaosClothAssetEditor3DViewportClient::SetClothEdMode(TObjectPtr<UChaosClothAssetEditorMode> InClothEdMode)
{
	ClothEdMode = InClothEdMode;
}

void FChaosClothAssetEditor3DViewportClient::SetClothEditorToolkit(TWeakPtr<const FChaosClothAssetEditorToolkit> InClothToolkit)
{
	ClothToolkit = InClothToolkit;
}

void FChaosClothAssetEditor3DViewportClient::SoftResetSimulation()
{
	if (ClothEdMode)
	{
		ClothEdMode->SoftResetSimulation();
	}
}

void FChaosClothAssetEditor3DViewportClient::HardResetSimulation()
{
	if (ClothEdMode)
	{
		ClothEdMode->HardResetSimulation();
	}
}

void FChaosClothAssetEditor3DViewportClient::SuspendSimulation()
{
	if (ClothEdMode)
	{
		ClothEdMode->SuspendSimulation();
	}
}

void FChaosClothAssetEditor3DViewportClient::ResumeSimulation()
{
	if (ClothEdMode)
	{
		ClothEdMode->ResumeSimulation();
	}
}

bool FChaosClothAssetEditor3DViewportClient::IsSimulationSuspended() const
{
	if (ClothEdMode)
	{
		return ClothEdMode->IsSimulationSuspended();
	}

	return false;
}

void FChaosClothAssetEditor3DViewportClient::SetEnableSimulation(bool bEnable)
{
	if (ClothEdMode)
	{
		ClothEdMode->SetEnableSimulation(bEnable);
	}
}

bool FChaosClothAssetEditor3DViewportClient::IsSimulationEnabled() const
{
	if (ClothEdMode)
	{
		return ClothEdMode->IsSimulationEnabled();
	}

	return false;
}

void FChaosClothAssetEditor3DViewportClient::SetLODLevel(int32 LODIndex)
{
	if (ClothEdMode)
	{
		ClothEdMode->SetLODModel(LODIndex);
	}
}

bool FChaosClothAssetEditor3DViewportClient::IsLODSelected(int32 LODIndex) const
{
	if (ClothEdMode)
	{
		return ClothEdMode->IsLODModelSelected(LODIndex);
	}
	return false;
}

int32 FChaosClothAssetEditor3DViewportClient::GetCurrentLOD() const
{
	if (ClothEdMode)
	{
		return ClothEdMode->GetLODModel();
	}
	return INDEX_NONE;
}

int32 FChaosClothAssetEditor3DViewportClient::GetLODCount() const
{
	if (ClothEdMode)
	{
		return ClothEdMode->GetNumLODs();
	}
	return 0;
}

void FChaosClothAssetEditor3DViewportClient::FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands)
{
	Commands.Add(FChaosClothAssetEditorCommands::Get().LODAuto);
	Commands.Add(FChaosClothAssetEditorCommands::Get().LOD0);
}

void FChaosClothAssetEditor3DViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);
	TSharedPtr<FClothEditorSimulationVisualization> Visualization = ClothEditorSimulationVisualization.Pin();
	UChaosClothComponent* const ClothComponent = GetPreviewClothComponent();
	if (Visualization && ClothComponent)
	{
		Visualization->DebugDrawSimulation(ClothComponent, PDI);
	}
}

void FChaosClothAssetEditor3DViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);
	TSharedPtr<FClothEditorSimulationVisualization> Visualization = ClothEditorSimulationVisualization.Pin();
	UChaosClothComponent* const ClothComponent = GetPreviewClothComponent();
	if (Visualization && ClothComponent)
	{
		Visualization->DebugDrawSimulationTexts(ClothComponent, &Canvas, &View);
	}
}

const UInputBehaviorSet* FChaosClothAssetEditor3DViewportClient::GetInputBehaviors() const
{
	return InputBehaviorSet;
}


// IClickBehaviorTarget
FInputRayHit FChaosClothAssetEditor3DViewportClient::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	// Here we are responding that we do want to handle click events, but we are only doing this so that we can also get drag events
	// TODO: Find out if there's a way we can just intercept mouse drag events and not single-click events
	return FInputRayHit(TNumericLimits<float>::Max());
}

void FChaosClothAssetEditor3DViewportClient::OnClicked(const FInputDeviceRay& ClickPos)
{
	// On a single click with no drag, respond as we would in ProcessClick()
	if (ClickPos.bHas2D)
	{
		HHitProxy* const HitProxy = Viewport->GetHitProxy(ClickPos.ScreenPosition[0], ClickPos.ScreenPosition[1]);
		UpdateSelection(HitProxy);
	}
}

// IClickDragBehaviorTarget
FInputRayHit FChaosClothAssetEditor3DViewportClient::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	// We do want to handle drag events
	return FInputRayHit(TNumericLimits<float>::Max());
}


FBox FChaosClothAssetEditor3DViewportClient::PreviewBoundingBox() const
{
	if (ClothEdMode)
	{
		return ClothEdMode->PreviewBoundingBox();
	}

	return FBox(ForceInitToZero);
}

TWeakPtr<FChaosClothPreviewScene> FChaosClothAssetEditor3DViewportClient::GetClothPreviewScene()
{
	return ClothPreviewScene;
}

TWeakPtr<const FChaosClothPreviewScene> FChaosClothAssetEditor3DViewportClient::GetClothPreviewScene() const
{
	return ClothPreviewScene;
}

UChaosClothComponent* FChaosClothAssetEditor3DViewportClient::GetPreviewClothComponent()
{
	if (ClothPreviewScene.IsValid())
	{
		return ClothPreviewScene.Pin()->GetClothComponent();
	}
	return nullptr;
}

const UChaosClothComponent* FChaosClothAssetEditor3DViewportClient::GetPreviewClothComponent() const
{
	if (ClothPreviewScene.IsValid())
	{
		return ClothPreviewScene.Pin()->GetClothComponent();
	}
	return nullptr;
}


void FChaosClothAssetEditor3DViewportClient::UpdateSelection(HHitProxy* HitProxy)
{
	const bool bIsShiftKeyDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
	const bool bIsCtrlKeyDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);

	USelection* SelectedComponents = ModeTools->GetSelectedComponents();

	TArray<UPrimitiveComponent*> PreviouslySelectedComponents;
	SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(PreviouslySelectedComponents);

	SelectedComponents->Modify();
	SelectedComponents->BeginBatchSelectOperation();

	SelectedComponents->DeselectAll();

	if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
	{
		const HActor* const ActorProxy = static_cast<HActor*>(HitProxy);
		if (ActorProxy && ActorProxy->Actor)
		{
			const AActor* const Actor = ActorProxy->Actor;
			
			Actor->ForEachComponent<UPrimitiveComponent>(true, [&](UPrimitiveComponent* Component)
			{
				SelectedComponents->Select(Component);
				Component->PushSelectionToProxy();
			});
		}
	}

	SelectedComponents->EndBatchSelectOperation();

	for (UPrimitiveComponent* const Component : PreviouslySelectedComponents)
	{
		Component->PushSelectionToProxy();
	}
}

void FChaosClothAssetEditor3DViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	FEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);
	UpdateSelection(HitProxy);
}


void FChaosClothAssetEditor3DViewportClient::ClearSelectedComponents()
{
	USelection* const SelectedComponents = ModeTools->GetSelectedComponents();
	TArray<UPrimitiveComponent*> PreviouslySelectedComponents;
	SelectedComponents->GetSelectedObjects<UPrimitiveComponent>(PreviouslySelectedComponents);

	for (UPrimitiveComponent* const Component : PreviouslySelectedComponents)
	{
		Component->PushSelectionToProxy();
	}

	SelectedComponents->DeselectAll();
}

void FChaosClothAssetEditor3DViewportClient::ComponentSelectionChanged(UObject* NewSelection)
{
	USelection* const SelectedComponents = ModeTools->GetSelectedComponents();

	// Update TransformProxy

	if (Gizmo && Gizmo->ActiveTarget)
	{
		DataBinder->UnbindFromGizmo(Gizmo, TransformProxy);
		Gizmo->ClearActiveTarget();
	}

	TransformProxy = NewObject<UTransformProxy>();
	TArray<USceneComponent*> Components;
	SelectedComponents->GetSelectedObjects(Components);
	for (USceneComponent* SelectedComponent : Components)
	{
		TransformProxy->AddComponent(SelectedComponent);
	}

	// Update gizmo
	if (Gizmo)
	{
		if (Components.Num() > 0)
		{
			Gizmo->SetActiveTarget(TransformProxy);
			Gizmo->SetVisibility(true);
			DataBinder->BindToInitializedGizmo(Gizmo, TransformProxy);
		}
		else
		{
			Gizmo->SetVisibility(false);
		}

		if (const TSharedPtr<FChaosClothPreviewScene> PinnedClothPreviewScene = ClothPreviewScene.Pin())
		{
			if (UChaosClothPreviewSceneDescription* const SceneDescription = PinnedClothPreviewScene->GetPreviewSceneDescription())
			{
				SceneDescription->bValidSelectionForTransform = (Components.Num() > 0);

				FPropertyChangedEvent Event(UChaosClothPreviewSceneDescription::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, bValidSelectionForTransform)));
				SceneDescription->PostEditChangeProperty(Event);
			}
		}

	}
}

} // namespace UE::Chaos::ClothAsset
