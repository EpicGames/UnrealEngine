// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "InputBehaviorSet.h"
#include "IPreviewLODController.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UChaosClothComponent;
class UChaosClothAssetEditorMode;
class UTransformProxy;
class UCombinedTransformGizmo;
class FTransformGizmoDataBinder;

namespace UE::Chaos::ClothAsset
{

class FChaosClothAssetEditorToolkit;
class FChaosClothPreviewScene;
class FClothEditorSimulationVisualization;

/**
 * Viewport client for the 3d sim preview in the cloth editor. Currently same as editor viewport
 * client but doesn't allow editor gizmos/widgets.
 */
class FChaosClothAssetEditor3DViewportClient : public FEditorViewportClient, 
	public TSharedFromThis<FChaosClothAssetEditor3DViewportClient>, 
	public IClickBehaviorTarget,
	public IClickDragBehaviorTarget,
	public IInputBehaviorSource,
	public IPreviewLODController
{
public:

	UE_API FChaosClothAssetEditor3DViewportClient(FEditorModeTools* InModeTools, TSharedPtr<FChaosClothPreviewScene> InPreviewScene, 
		TSharedPtr<FClothEditorSimulationVisualization> InVisualization,
		const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	// Call this after construction to initialize callbacks when settings change
	UE_API void RegisterDelegates();

	UE_API virtual ~FChaosClothAssetEditor3DViewportClient();

	// Delete the viewport gizmo and transform proxy
	UE_API void DeleteViewportGizmo();

	UE_API void ClearSelectedComponents();

	void EnableSimMeshWireframe(bool bEnable) { bSimMeshWireframe = bEnable; }
	bool SimMeshWireframeEnabled() const { return bSimMeshWireframe; }

	UE_API void EnableRenderMeshWireframe(bool bEnable);
	bool RenderMeshWireframeEnabled() const { return bRenderMeshWireframe; }

	UE_API void SetClothEdMode(TObjectPtr<UChaosClothAssetEditorMode> ClothEdMode);
	UE_API void SetClothEditorToolkit(TWeakPtr<const FChaosClothAssetEditorToolkit> ClothToolkit);

	UE_API void SoftResetSimulation();
	UE_API void HardResetSimulation();
	UE_API void SuspendSimulation();
	UE_API void ResumeSimulation();
	UE_API bool IsSimulationSuspended() const;
	UE_API void SetEnableSimulation(bool bEnable);
	UE_API bool IsSimulationEnabled() const;

	// IPreviewLODController interface
	// LODIndex == INDEX_NONE is LOD Auto
	UE_API virtual void SetLODLevel(int32 LODIndex) override;
	UE_API virtual int32 GetLODCount() const override;
	UE_API virtual int32 GetCurrentLOD() const override;
	UE_API virtual bool IsLODSelected(int32 LODIndex) const override;
	virtual int32 GetAutoLODStartingIndex() const override { return 1; }
	UE_API virtual void FillLODCommands(TArray<TSharedPtr<FUICommandInfo>>& Commands) override;
	// ~IPreviewLODController

	UE_API FBox PreviewBoundingBox() const;

	UE_API TWeakPtr<FChaosClothPreviewScene> GetClothPreviewScene();
	UE_API TWeakPtr<const FChaosClothPreviewScene> GetClothPreviewScene() const;
	UE_API UChaosClothComponent* GetPreviewClothComponent();
	UE_API const UChaosClothComponent* GetPreviewClothComponent() const;
	TWeakPtr<FClothEditorSimulationVisualization> GetSimulationVisualization() {
		return ClothEditorSimulationVisualization;
	}
	TWeakPtr<const FChaosClothAssetEditorToolkit> GetClothToolKit() const { return ClothToolkit; }

private:

	// FGCObject override
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	// FEditorViewportClient overrides
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual void SetWidgetMode(UE::Widget::EWidgetMode NewMode) override {}
	virtual UE::Widget::EWidgetMode GetWidgetMode() const override { return UE::Widget::EWidgetMode::WM_None; }
	UE_API virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;
	UE_API virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;

	// IClickBehaviorTarget
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IClickDragBehaviorTarget
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override {}
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override {}
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override {}
	virtual void OnTerminateDragSequence() override {}

	// IInputBehaviorSource
	UE_API virtual const UInputBehaviorSet* GetInputBehaviors() const override;

	UE_API void ComponentSelectionChanged(UObject* NewSelection);

	// Update the selected components based on hitproxy
	UE_API void UpdateSelection(HHitProxy* HitProxy);

	TWeakPtr<FChaosClothPreviewScene> ClothPreviewScene;

	TObjectPtr<UChaosClothAssetEditorMode> ClothEdMode;

	TWeakPtr<const FChaosClothAssetEditorToolkit> ClothToolkit;

	TWeakPtr<FClothEditorSimulationVisualization> ClothEditorSimulationVisualization;
	
	bool bSimMeshWireframe = true;
	bool bRenderMeshWireframe = false;

	// Dataflow render support
	UE::Dataflow::FTimestamp LastModifiedTimestamp = UE::Dataflow::FTimestamp::Invalid;

	// Gizmo support
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;
	TObjectPtr<UCombinedTransformGizmo> Gizmo = nullptr;
	TSharedPtr<FTransformGizmoDataBinder> DataBinder = nullptr;

	TObjectPtr<UInputBehaviorSet> InputBehaviorSet;
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
