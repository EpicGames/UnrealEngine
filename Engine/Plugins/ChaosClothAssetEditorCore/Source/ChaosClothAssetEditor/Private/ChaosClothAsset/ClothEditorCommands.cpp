// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorStyle.h"
#include "ChaosClothAsset/ClothEditorToolBuilders.h"

#define LOCTEXT_NAMESPACE "FChaosClothAssetEditorCommands"

namespace UE::Chaos::ClothAsset
{

const FString FChaosClothAssetEditorCommands::BeginRemeshToolIdentifier = TEXT("BeginRemeshTool");
const FString FChaosClothAssetEditorCommands::BeginAttributeEditorToolIdentifier = TEXT("BeginAttributeEditorTool");
const FString FChaosClothAssetEditorCommands::BeginWeightMapPaintToolIdentifier = TEXT("BeginWeightMapPaintTool");
const FString FChaosClothAssetEditorCommands::AddWeightMapNodeIdentifier = TEXT("AddWeightMapNode");
const FString FChaosClothAssetEditorCommands::BeginTransferSkinWeightsToolIdentifier = TEXT("BeginTransferSkinWeightsTool");
const FString FChaosClothAssetEditorCommands::AddTransferSkinWeightsNodeIdentifier = TEXT("AddTransferSkinWeightsNode");
const FString FChaosClothAssetEditorCommands::BeginMeshSelectionToolIdentifier = TEXT("BeginMeshSelectionTool");
const FString FChaosClothAssetEditorCommands::AddMeshSelectionNodeIdentifier = TEXT("AddMeshSelectionNode");
const FString FChaosClothAssetEditorCommands::ToggleSimulationSuspendedIdentifier = TEXT("ToggleSimulationSuspended");
const FString FChaosClothAssetEditorCommands::SoftResetSimulationIdentifier = TEXT("SoftResetSimulation");
const FString FChaosClothAssetEditorCommands::HardResetSimulationIdentifier = TEXT("HardResetSimulation");
const FString FChaosClothAssetEditorCommands::TogglePreviewWireframeIdentifier = TEXT("TogglePreviewWireframe");
const FString FChaosClothAssetEditorCommands::ToggleConstructionViewWireframeIdentifier = TEXT("ToggleConstructionViewWireframe");
const FString FChaosClothAssetEditorCommands::ToggleConstructionViewSeamsIdentifier = TEXT("ToggleConstructionViewSeams");
const FString FChaosClothAssetEditorCommands::ToggleConstructionViewSeamsCollapseIdentifier = TEXT("ToggleConstructionViewSeamsCollapse");
const FString FChaosClothAssetEditorCommands::ToggleConstructionViewSurfaceNormalsIdentifier = TEXT("ToggleConstructionViewSurfaceNormals");

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For OpenClothEditor
FChaosClothAssetEditorCommands::FChaosClothAssetEditorCommands()
	: TBaseCharacterFXEditorCommands<FChaosClothAssetEditorCommands>("ChaosClothAssetEditor",
		LOCTEXT("ContextDescription", "Cloth Editor"), 
		NAME_None, // Parent
		FChaosClothAssetEditorStyle::Get().GetStyleSetName())
{
}

FChaosClothAssetEditorCommands::~FChaosClothAssetEditorCommands() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FChaosClothAssetEditorCommands::RegisterCommands()
{
	TBaseCharacterFXEditorCommands::RegisterCommands();

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	UI_COMMAND(OpenClothEditor, "Cloth Editor", "Open the Cloth Editor window", EUserInterfaceActionType::Button, FInputChord());  // This looks redundant, it should have probably be deprecated when OpenClothAssetInClothPanelEditor was added
	UI_COMMAND(OpenClothAssetInClothPanelEditor, "Edit in Cloth Panel Editor (Deprecated)", "Open the Cloth Asset in the deprecated Cloth Panel Editor instead of the unified Dataflow one", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenClothAssetInDataflowEditor, "Edit in Dataflow Editor", "Open the Cloth Asset in the Dataflow Editor", EUserInterfaceActionType::Button, FInputChord());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UI_COMMAND(BeginRemeshTool, "Remesh", "Remesh the selected mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BeginAttributeEditorTool, "AttrEd", "Edit/configure mesh attributes", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(BeginWeightMapPaintTool, "Weight Map", "Paint weight maps on the mesh", EUserInterfaceActionType::None, FInputChord());
	UI_COMMAND(AddWeightMapNode, "Weight Map", "Paint weight maps on the mesh", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BeginMeshSelectionTool, "Select", "Select mesh elements", EUserInterfaceActionType::None, FInputChord());
	UI_COMMAND(AddMeshSelectionNode, "Select", "Select mesh elements", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BeginTransferSkinWeightsTool, "Transfer Skin Weights", "Transfer skinning weights from a SkeletalMesh", EUserInterfaceActionType::None, FInputChord());
	UI_COMMAND(AddTransferSkinWeightsNode, "Transfer Skin Weights", "Transfer skinning weights from a SkeletalMesh", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SetConstructionMode2D, "2D Sim", "Switches the viewport to 2D simulation mesh view", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetConstructionMode3D, "3D Sim", "Switches the viewport to 3D simulation mesh view", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetConstructionModeRender, "Render", "Switches the viewport to render mesh view", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(TogglePreviewWireframe, "Preview Wireframe", "Toggle preview wireframe", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleConstructionViewWireframe, "Construction View Wireframe", "Toggle construction view wireframe", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleConstructionViewSeams, "Show Seams", "Display seam information (not available for non-manifold meshes).", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleConstructionViewSeamsCollapse, "Collapse Seam Lines", "Display a single line connecting each seam, rather than all stitches.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleConstructionViewSurfaceNormals, "Show Normals", "Display surface normals", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(TogglePatternColor, "Color Patterns", "Display each Pattern in a different color", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleMeshStats, "Mesh Stats", "Show mesh stats in the viewport.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(SoftResetSimulation, "Soft Reset Simulation", "Soft reset the cloth simulation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(HardResetSimulation, "Hard Reset Simulation", "Hard reset the cloth simulation", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::C));
	UI_COMMAND(ToggleSimulationSuspended, "Toggle Simulation", "Toggle the simulation of the cloth", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(LODAuto, "LOD Auto", "Automatically select LOD", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LOD0, "LOD 0", "Force select LOD 0", EUserInterfaceActionType::RadioButton, FInputChord());
}

void FChaosClothAssetEditorCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{
	GetClothEditorToolDefaultObjectList(ToolCDOs);
}

void FChaosClothAssetEditorCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
	if (FChaosClothAssetEditorCommands::IsRegistered())
	{
		if (bUnbind)
		{
			FChaosClothAssetEditorCommands::Get().UnbindActiveCommands(UICommandList);
		}
		else
		{
			FChaosClothAssetEditorCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
		}
	}
}
} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
