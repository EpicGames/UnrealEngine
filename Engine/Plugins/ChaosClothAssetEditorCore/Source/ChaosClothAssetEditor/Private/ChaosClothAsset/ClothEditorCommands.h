// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorCommands.h"

namespace UE::Chaos::ClothAsset
{
class CHAOSCLOTHASSETEDITOR_API FChaosClothAssetEditorCommands : public TBaseCharacterFXEditorCommands<FChaosClothAssetEditorCommands>
{
public:

	FChaosClothAssetEditorCommands();
	~FChaosClothAssetEditorCommands();

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For OpenClothEditor
	FChaosClothAssetEditorCommands(const FChaosClothAssetEditorCommands&) = default;
	FChaosClothAssetEditorCommands(FChaosClothAssetEditorCommands&&) = default;
	FChaosClothAssetEditorCommands& operator=(const FChaosClothAssetEditorCommands&) = default;
	FChaosClothAssetEditorCommands& operator=(FChaosClothAssetEditorCommands&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// TBaseCharacterFXEditorCommands<> interface
	virtual void RegisterCommands() override;

	// TInteractiveToolCommands<>
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;

	/**
	 * Add or remove commands relevant to Tool to the given UICommandList.
	 * Call this when the active tool changes (eg on ToolManager.OnToolStarted / OnToolEnded)
	 * @param bUnbind if true, commands are removed, otherwise added
	 */
	static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);


public:

	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	TSharedPtr<FUICommandInfo> OpenClothEditor;
	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	TSharedPtr<FUICommandInfo> OpenClothAssetInClothPanelEditor;
	UE_DEPRECATED(5.8, "The Cloth Panel Editor is now deprecated, use the Dataflow Editor instead.")
	TSharedPtr<FUICommandInfo> OpenClothAssetInDataflowEditor;  // The Dataflow Editor command is no longer needed once the deprecation is completed

	const static FString BeginRemeshToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginRemeshTool;

	const static FString BeginWeightMapPaintToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginWeightMapPaintTool;
	const static FString AddWeightMapNodeIdentifier;
	TSharedPtr<FUICommandInfo> AddWeightMapNode;

	const static FString BeginAttributeEditorToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginAttributeEditorTool;

	const static FString BeginTransferSkinWeightsToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginTransferSkinWeightsTool;
	const static FString AddTransferSkinWeightsNodeIdentifier;
	TSharedPtr<FUICommandInfo> AddTransferSkinWeightsNode;

	const static FString BeginMeshSelectionToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginMeshSelectionTool;
	const static FString AddMeshSelectionNodeIdentifier;
	TSharedPtr<FUICommandInfo> AddMeshSelectionNode;

	// Construction viewport commands
	const static FString ToggleConstructionViewWireframeIdentifier;
	TSharedPtr<FUICommandInfo> ToggleConstructionViewWireframe;

	const static FString ToggleConstructionViewSeamsIdentifier;
	TSharedPtr<FUICommandInfo> ToggleConstructionViewSeams;

	const static FString ToggleConstructionViewSeamsCollapseIdentifier;
	TSharedPtr<FUICommandInfo> ToggleConstructionViewSeamsCollapse;

	TSharedPtr<FUICommandInfo> TogglePatternColor;
	TSharedPtr<FUICommandInfo> ToggleMeshStats;

	const static FString ToggleConstructionViewSurfaceNormalsIdentifier;
	TSharedPtr<FUICommandInfo> ToggleConstructionViewSurfaceNormals;

	TSharedPtr<FUICommandInfo> SetConstructionMode2D;
	TSharedPtr<FUICommandInfo> SetConstructionMode3D;
	TSharedPtr<FUICommandInfo> SetConstructionModeRender;

	// Preview viewport commands
	const static FString TogglePreviewWireframeIdentifier;
	TSharedPtr<FUICommandInfo> TogglePreviewWireframe;

	const static FString SoftResetSimulationIdentifier;
	TSharedPtr<FUICommandInfo> SoftResetSimulation;

	const static FString HardResetSimulationIdentifier;
	TSharedPtr<FUICommandInfo> HardResetSimulation;

	const static FString ToggleSimulationSuspendedIdentifier;
	TSharedPtr<FUICommandInfo> ToggleSimulationSuspended;

	TSharedPtr<FUICommandInfo> LODAuto;
	TSharedPtr<FUICommandInfo> LOD0;
};
} // namespace UE::Chaos::ClothAsset
