// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorModeToolkit.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothEditorRestSpaceViewportClient.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "ChaosClothAsset/ClothEditorToolBuilders.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "AssetEditorModeManager.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "EditorViewportClient.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EngineAnalytics.h"
#include "Framework/Commands/UICommandList.h"
#include "InteractiveTool.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewScene.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "ToolTargetManager.h"
#include "Toolkits/BaseToolkit.h"
#include "ToolTargets/ToolTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "Engine/Selection.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "RemeshMeshTool.h"
#include "AttributeEditorTool.h"
#include "MeshAttributePaintTool.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMeshEditor.h"
#include "Settings/EditorStyleSettings.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "ClothingAssetFactoryInterface.h"
#include "ClothingAssetFactory.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Utils/ClothingMeshUtils.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "BoneWeights.h"
#include "ToolSetupUtil.h"
#include "Dataflow/DataflowComponent.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "ContextObjectStore.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Dataflow/DataflowGraphSchemaAction.h"
#include "Components/SkeletalMeshComponent.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "DynamicMesh/MeshNormals.h"
#include "Dataflow/DataflowRenderingFactory.h"	// For Dataflow View Modes

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorMode)

#define LOCTEXT_NAMESPACE "UChaosClothAssetEditorMode"

const FEditorModeID UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId = TEXT("EM_ChaosClothAssetEditorMode");


namespace UE::Chaos::ClothAsset::Private
{
	bool bClothEditorEnableToolsInPIE = true;
	FAutoConsoleVariableRef CVARClothEditorEnableToolsInPIE(TEXT("p.ChaosCloth.EnableToolsInPIE"), bClothEditorEnableToolsInPIE,
		TEXT("Enable Cloth Editor tools while Play In Editor is running [def:true]"));

	void RemoveClothWeightMaps(UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade, const TArray<FName>& WeightMapNames)
	{
		for (const FName& WeightMapName : WeightMapNames)
		{
			if (ClothFacade.HasWeightMap(WeightMapName))
			{
				ClothFacade.RemoveWeightMap(WeightMapName);
			}
		}
	}

	TArray<FName> GetDynamicMeshWeightMapNames(const UE::Geometry::FDynamicMesh3& DynamicMesh)
	{
		TArray<FName> OutWeightMapNames;

		for (int32 DynamicMeshWeightMapIndex = 0; DynamicMeshWeightMapIndex < DynamicMesh.Attributes()->NumWeightLayers(); ++DynamicMeshWeightMapIndex)
		{
			const UE::Geometry::FDynamicMeshWeightAttribute* const WeightMapAttribute = DynamicMesh.Attributes()->GetWeightLayer(DynamicMeshWeightMapIndex);
			const FName WeightMapName = WeightMapAttribute->GetName();
			OutWeightMapNames.Add(WeightMapName);
		}

		return OutWeightMapNames;
	}

	FLinearColor PseudoRandomColor(int32 NumColorRotations)
	{
		constexpr uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
		uint8 Seed = Spread;
		for (int32 Rotation = 0; Rotation < NumColorRotations; ++Rotation)
		{
			Seed += Spread;
		}
		return FLinearColor::MakeFromHSV8(Seed, 180, 140);
	}

	FString GetToolName(const UInteractiveTool& Tool)
	{
		const FString* const ToolName = FTextInspector::GetSourceString(Tool.GetToolInfo().ToolDisplayName);
		return ToolName ? *ToolName : FString(TEXT("<Invalid ToolName>"));
	}
}


UChaosClothAssetEditorMode::UChaosClothAssetEditorMode()
{
	Info = FEditorModeInfo(
		EM_ChaosClothAssetEditorModeId,
		LOCTEXT("ChaosClothAssetEditorModeName", "Cloth"),
		FSlateIcon(),
		false);
}

const FToolTargetTypeRequirements& UChaosClothAssetEditorMode::GetToolTargetRequirements()
{
	static const FToolTargetTypeRequirements ToolTargetRequirements =
		FToolTargetTypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshCommitter::StaticClass(),
			UDynamicMeshProvider::StaticClass()
			});
	return ToolTargetRequirements;
}

void UChaosClothAssetEditorMode::Enter()
{
	UBaseCharacterFXEditorMode::Enter();

	// Register gizmo ContextObject for use inside interactive tools
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());


	//
	// Engine Analytics
	//

	// Log mode starting
	if (FEngineAnalytics::IsAvailable())
	{
		LastModeStartTimestamp = FDateTime::UtcNow();
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), LastModeStartTimestamp.ToString()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ChaosClothEditor.Enter"), EventAttributes);

		// Log tool starting
		GetToolManager()->OnToolStarted.AddLambda([this](UInteractiveToolManager* Manager, UInteractiveTool* Tool)
		{
			if (FEngineAnalytics::IsAvailable() && Tool)
			{
				LastToolStartTimestamp = FDateTime::UtcNow();

				TArray<FAnalyticsEventAttribute> EventAttributes;
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ToolName"), UE::Chaos::ClothAsset::Private::GetToolName(*Tool)));
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), LastToolStartTimestamp.ToString()));

				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ChaosClothEditor.ToolStarted"), EventAttributes);
			}
		});

		// Log tool ending
		GetToolManager()->OnToolEnded.AddLambda([this](UInteractiveToolManager* Manager, UInteractiveTool* Tool)
		{
			if (FEngineAnalytics::IsAvailable() && Tool)
			{
				const FDateTime Now = FDateTime::UtcNow();
				const FTimespan ToolUsageDuration = Now - LastToolStartTimestamp;

				TArray<FAnalyticsEventAttribute> EventAttributes;
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ToolName"), UE::Chaos::ClothAsset::Private::GetToolName(*Tool)));
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), Now.ToString()));
				EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ToolUsageDuration.GetTotalSeconds())));

				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ChaosClothEditor.ToolEnded"), EventAttributes);
			}
		});
	}
}

void UChaosClothAssetEditorMode::AddToolTargetFactories()
{
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(GetToolManager()));
}

void UChaosClothAssetEditorMode::RegisterClothTool(TSharedPtr<FUICommandInfo> UICommand, 
	FString ToolIdentifier, 
	UInteractiveToolBuilder* Builder, 
	const IChaosClothAssetEditorToolBuilder* ClothToolBuilder,
	UEditorInteractiveToolsContext* const ToolsContext, 
	EToolsContextScope ToolScope)
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	if (!ToolsContext)
	{
		return;
	}

	if (ToolScope == EToolsContextScope::Default)
	{
		ToolScope = GetDefaultToolScope();
	}
	ensure(ToolScope != EToolsContextScope::Editor);

	ToolsContext->ToolManager->RegisterToolType(ToolIdentifier, Builder);

	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(UICommand,
		FExecuteAction::CreateWeakLambda(ToolsContext, [this, ToolsContext, ToolIdentifier, ClothToolBuilder]()
		{
			// Check if we need to switch view modes before starting the tool
			TArray<UE::Chaos::ClothAsset::EClothPatternVertexType> SupportedModes;

			const UDataflowContextObject* EditorContextObject = ToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
			if (!EditorContextObject)
			{
				InitializeContextObject();
				EditorContextObject = ToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
			}
			checkf(EditorContextObject, TEXT("Failed to find or create ContextObject"));
			ClothToolBuilder->GetSupportedViewModes(*EditorContextObject, SupportedModes);

			bDynamicMeshUseInputCollection = true;
			if (SupportedModes.Num() > 0 && !SupportedModes.Contains(this->GetConstructionViewMode()))
			{
				if (!bShouldRestoreSavedConstructionViewMode)
				{
					// remember the current view mode so we can restore it later
					SavedConstructionViewMode = this->GetConstructionViewMode();	
					bShouldRestoreSavedConstructionViewMode = true;
				}

				// switch to the preferred view mode for the tool that's about to start
				this->SetConstructionViewMode(SupportedModes[0]);
			}
			else
			{
				const bool bCurrentDynamicMeshIsInput = EditorContextObject->IsUsingInputCollection();

				if (!bCurrentDynamicMeshIsInput || bDynamicMeshComponentInitDeferred)
				{
					ReinitializeDynamicMeshComponents();
				}
			}

			// Check if we need to disable wireframe mode before starting tool.
			const bool bCanSetWireframeActive = ClothToolBuilder->CanSetConstructionViewWireframeActive();
			if (!bCanSetWireframeActive)
			{
				if (!bShouldRestoreConstructionViewWireframe)
				{
					bShouldRestoreConstructionViewWireframe = bConstructionViewWireframe;
				}
				bConstructionViewWireframe = false;
			}

			// Seams
			if (!bShouldRestoreConstructionViewSeams)
			{
				bShouldRestoreConstructionViewSeams = bConstructionViewSeamsVisible;
			}
			bConstructionViewSeamsVisible = false;

			ActiveToolsContext = ToolsContext;
			ToolsContext->StartTool(ToolIdentifier);
		}),
		FCanExecuteAction::CreateWeakLambda(ToolsContext, [this, ToolIdentifier, ToolsContext]() 
		{
			return ShouldToolStartBeAllowed(ToolIdentifier) &&
			ToolsContext->ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
		}),
		FIsActionChecked::CreateUObject(ToolsContext, &UEdModeInteractiveToolsContext::IsToolActive, EToolSide::Mouse, ToolIdentifier),
		EUIActionRepeatMode::RepeatDisabled);

}

void UChaosClothAssetEditorMode::AddNode(FName NewNodeType)
{
	const FName ConnectionType = FManagedArrayCollection::StaticType();
	const FName ConnectionName("Collection");

	UEdGraphNode* const CurrentlySelectedNode = GetSingleSelectedNodeWithOutputType(ConnectionType);
	checkf(CurrentlySelectedNode, TEXT("No node with FManagedArrayCollection output is currently selected in the Dataflow graph"));

	const UEdGraphNode* const NewNode = CreateAndConnectNewNode(NewNodeType, *CurrentlySelectedNode, ConnectionType, ConnectionName);
	verifyf(NewNode, TEXT("Failed to create a new node: %s"), *NewNodeType.ToString());

	StartToolForSelectedNode(NewNode);
}

bool UChaosClothAssetEditorMode::CanAddNode(FName NewNodeType) const
{
	const UEdGraphNode* const CurrentlySelectedNode = GetSingleSelectedNodeWithOutputType(FManagedArrayCollection::StaticType());
	return (CurrentlySelectedNode != nullptr);
}

void UChaosClothAssetEditorMode::RegisterAddNodeCommand(TSharedPtr<FUICommandInfo> AddNodeCommand, const FName& NewNodeType, TSharedPtr<FUICommandInfo> StartToolCommand)
{
	// ToolkitCommands->MapAction(AddNodeCommand) is done in FChaosClothAssetEditorToolkit

	NodeTypeToAddNodeCommandMap.Add(NewNodeType, AddNodeCommand);
	NodeTypeToToolCommandMap.Add(NewNodeType, StartToolCommand);
}


void UChaosClothAssetEditorMode::RegisterTools()
{
	using namespace UE::Chaos::ClothAsset;

	const FChaosClothAssetEditorCommands& CommandInfos = FChaosClothAssetEditorCommands::Get();

	UEditorInteractiveToolsContext* const ConstructionViewportToolsContext = GetInteractiveToolsContext();

	UClothEditorWeightMapPaintToolBuilder* WeightMapPaintToolBuilder = NewObject<UClothEditorWeightMapPaintToolBuilder>();
	RegisterClothTool(CommandInfos.BeginWeightMapPaintTool, FChaosClothAssetEditorCommands::BeginWeightMapPaintToolIdentifier, WeightMapPaintToolBuilder, WeightMapPaintToolBuilder, ConstructionViewportToolsContext);
	RegisterAddNodeCommand(CommandInfos.AddWeightMapNode, FChaosClothAssetWeightMapNode::StaticType(), CommandInfos.BeginWeightMapPaintTool);

	UClothTransferSkinWeightsToolBuilder* TransferToolBuilder = NewObject<UClothTransferSkinWeightsToolBuilder>();
	RegisterClothTool(CommandInfos.BeginTransferSkinWeightsTool, FChaosClothAssetEditorCommands::BeginTransferSkinWeightsToolIdentifier, TransferToolBuilder, TransferToolBuilder, ConstructionViewportToolsContext);
	RegisterAddNodeCommand(CommandInfos.AddTransferSkinWeightsNode, FChaosClothAssetTransferSkinWeightsNode::StaticType(), CommandInfos.BeginTransferSkinWeightsTool);

	UClothMeshSelectionToolBuilder* SelectionToolBuilder = NewObject<UClothMeshSelectionToolBuilder>();
	RegisterClothTool(CommandInfos.BeginMeshSelectionTool, FChaosClothAssetEditorCommands::BeginMeshSelectionToolIdentifier, SelectionToolBuilder, SelectionToolBuilder, ConstructionViewportToolsContext);
	RegisterAddNodeCommand(CommandInfos.AddMeshSelectionNode, FChaosClothAssetSelectionNode_v2::StaticType(), CommandInfos.BeginMeshSelectionTool);
}

bool UChaosClothAssetEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	// Allow switching away from tool if no changes have been made in the tool yet (which we infer from the CanAccept status)
	if (GetInteractiveToolsContext()->CanAcceptActiveTool())
	{
		return false;
	}
	
	if (UE::Chaos::ClothAsset::Private::bClothEditorEnableToolsInPIE)
	{
		// UEdMode::ShouldToolStartBeAllowed returns (!GEditor->PlayWorld && !GIsPlayInEditorWorld) but we want to allow tools to start while in PIE
		return true;
	}
	else
	{
		return UBaseCharacterFXEditorMode::ShouldToolStartBeAllowed(ToolIdentifier);
	}
}

void UChaosClothAssetEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<UE::Chaos::ClothAsset::FChaosClothAssetEditorModeToolkit>();
}

void UChaosClothAssetEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	using namespace UE::Chaos::ClothAsset;

	FChaosClothAssetEditorCommands::UpdateToolCommandBinding(Tool, ToolCommandList, false);
}

void UChaosClothAssetEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	UE::Chaos::ClothAsset::FChaosClothAssetEditorCommands::UpdateToolCommandBinding(Tool, ToolCommandList, true);

	if (bShouldRestoreConstructionViewWireframe)
	{
		bConstructionViewWireframe = true;
		bShouldRestoreConstructionViewWireframe = false;
	}

	if (bShouldRestoreConstructionViewSeams)
	{
		bConstructionViewSeamsVisible = true;
		bShouldRestoreConstructionViewSeams = false;
	}

	bDynamicMeshUseInputCollection = false;
	if (bShouldRestoreSavedConstructionViewMode)
	{
		SetConstructionViewMode(SavedConstructionViewMode);
		bShouldRestoreSavedConstructionViewMode = false;
	}
	else
	{
		ReinitializeDynamicMeshComponents();
	}

	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		GraphEditor->SetEnabled(true);
	}
}

void UChaosClothAssetEditorMode::BindCommands()
{
	using namespace UE::Chaos::ClothAsset;
	const FChaosClothAssetEditorCommands& CommandInfos = FChaosClothAssetEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// Hook up to Enter/Esc key presses
	CommandList->MapAction(
		CommandInfos.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { AcceptActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
			return GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool();
		}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	CommandList->MapAction(
		CommandInfos.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { CancelActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
			return GetInteractiveToolsContext()->CanCompleteActiveTool() || GetInteractiveToolsContext()->CanCancelActiveTool();
		}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);
}


void UChaosClothAssetEditorMode::Exit()
{
	UActorComponent::MarkRenderStateDirtyEvent.RemoveAll(this);

	if (DynamicMeshComponent)
	{
		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->SelectionOverrideDelegate.Unbind();
	}
	DynamicMeshComponent = nullptr;
	DynamicMeshComponentParentActor = nullptr;

	if (WireframeDraw)
	{
		WireframeDraw->Disconnect();
	}
	WireframeDraw = nullptr;

	if (ClothSeamDraw)
	{
		ClothSeamDraw->Disconnect();
	}
	ClothSeamDraw = nullptr;

	if (SurfaceNormalDraw)
	{
		SurfaceNormalDraw->Disconnect();
	}
	SurfaceNormalDraw = nullptr;

	PropertyObjectsToTick.Empty();
	PreviewScene = nullptr;

	//
	// Engine Analytics
	//
	// Log mode exit
	if (FEngineAnalytics::IsAvailable())
	{
		const FTimespan ModeUsageDuration = FDateTime::UtcNow() - LastModeStartTimestamp;

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Timestamp"), FDateTime::UtcNow().ToString()));
		Attributes.Add(FAnalyticsEventAttribute(TEXT("Duration.Seconds"), static_cast<float>(ModeUsageDuration.GetTotalSeconds())));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ChaosClothEditor.Exit"));
	}

	Super::Exit();
}

void UChaosClothAssetEditorMode::SetPreviewScene(UE::Chaos::ClothAsset::FChaosClothPreviewScene* InPreviewScene)
{
	PreviewScene = InPreviewScene;
}

void UChaosClothAssetEditorMode::CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) 
{
	// TODO: When we have a cloth component tool target, create it here

}


bool UChaosClothAssetEditorMode::IsComponentSelected(const UPrimitiveComponent* InComponent)
{
	if (const FEditorModeTools* const ModeManager = GetModeManager())
	{
		if (const UTypedElementSelectionSet* const TypedElementSelectionSet = ModeManager->GetEditorSelectionSet())
		{
			if (const FTypedElementHandle ComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent))
			{
				const bool bElementSelected = TypedElementSelectionSet->IsElementSelected(ComponentElement, FTypedElementIsSelectedOptions());
				return bElementSelected;
			}
		}
	}

	return false;
}


void UChaosClothAssetEditorMode::SetSelectedClothCollection(TSharedPtr<FManagedArrayCollection> Collection, TSharedPtr<FManagedArrayCollection> InputCollection, bool bDeferDynamicMeshInitForTool)
{
	SelectedClothCollection = Collection;
	SelectedInputClothCollection = InputCollection;

	if (bDeferDynamicMeshInitForTool)
	{
		bDynamicMeshComponentInitDeferred = true;
	}
	else
	{
		ReinitializeDynamicMeshComponents();

		// The first time we get a valid mesh, refocus the camera on it
		FirstTimeFocusRestSpaceViewport();
	}
}

TSharedPtr<FManagedArrayCollection> UChaosClothAssetEditorMode::GetClothCollection()
{
	return SelectedClothCollection;

	// TODO: If no cloth collection node is selected, show the ClothAsset's collection. In this case, also ensure that any interactive tools are disabled. (UE-181574)
}

TSharedPtr<FManagedArrayCollection> UChaosClothAssetEditorMode::GetInputClothCollection()
{
	return SelectedInputClothCollection;

	// TODO: If no cloth collection node is selected, show the ClothAsset's collection. In this case, also ensure that any interactive tools are disabled. (UE-181574)
}


void UChaosClothAssetEditorMode::InitializeSeamDraw()
{
	if (!ClothSeamDraw)
	{
		return;
	}

	ClothSeamDraw->RemoveAllLineSets();

	if (!bConstructionViewSeamsVisible)
	{
		return;
	}

	if (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Render)
	{
		return;
	}

	if (!DynamicMeshComponent || !DynamicMeshComponent->GetMesh())
	{
		return;
	}

	const TSharedPtr<const FManagedArrayCollection> Collection = GetClothCollection();
	if (!Collection)
	{
		return;
	}

	const UE::Geometry::FDynamicMesh3& Mesh = *DynamicMeshComponent->GetMesh();

	// Seam view not available on non-manifold meshes
	const UE::Geometry::FNonManifoldMappingSupport NonManifold(Mesh);
	if (NonManifold.IsNonManifoldVertexInSource())
	{
		bConstructionViewSeamsVisible = false;
		return;
	}

	const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection.ToSharedRef());

	ULineSetComponent* const Lines = ClothSeamDraw->AddLineSet("SeamLines");
	const bool bDepthTested = (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D);
	Lines->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager(), bDepthTested));
	const float LineThickness = (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D) ? 4.0f : 2.0f;

	UPointSetComponent* const Points = ClothSeamDraw->AddPointSet("SeamPoints");
	constexpr float PointSize = 4.0f;


	int32 ConnectedSeamIndex = 0;		// Used to generate different colors for each connected seam, if multiple connected seams are found per input seam

	for (int32 SeamIndex = 0; SeamIndex < ClothFacade.GetNumSeams(); ++SeamIndex)
	{
		const UE::Chaos::ClothAsset::FCollectionClothSeamConstFacade SeamFacade = ClothFacade.GetSeam(SeamIndex);

		if (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D)
		{
			// Stitches are given in random order, so first construct paths of connected stitches
			// Note one SeamFacade can contain multiple disjoint paths
			TArray<TArray<FIntVector2>> ConnectedSeams;
			UE::Chaos::ClothAsset::FClothGeometryTools::BuildConnectedSeams2D(Collection.ToSharedRef(), SeamIndex, Mesh, ConnectedSeams);

			for (const TArray<FIntVector2>& ConnectedSeam : ConnectedSeams)
			{
				const FColor SeamColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(ConnectedSeamIndex++).ToFColor(true);

				// draw connected edge on each side of the seam
				for (int32 StitchID = 0; StitchID < ConnectedSeam.Num() - 1; ++StitchID)
				{
					const FVector3d PointA = Mesh.GetVertex(ConnectedSeam[StitchID][0]);
					const FVector3d PointB = Mesh.GetVertex(ConnectedSeam[StitchID][1]);
					const FVector3d PointC = Mesh.GetVertex(ConnectedSeam[StitchID+1][0]);
					const FVector3d PointD = Mesh.GetVertex(ConnectedSeam[StitchID+1][1]);
					Lines->AddLine(PointA, PointC, SeamColor, LineThickness);
					Lines->AddLine(PointB, PointD, SeamColor, LineThickness);
				}

				// draw connection between stitch points
				if (bConstructionViewSeamsCollapse)
				{
					const int32 StitchID = ConnectedSeam.Num() / 2;
					const int32 VertexA = ConnectedSeam[StitchID][0];
					const int32 VertexB = ConnectedSeam[StitchID][1];
					const FVector PtA = Mesh.GetVertex(VertexA);
					const FVector PtB = Mesh.GetVertex(VertexB);
					Lines->AddLine(PtA, PtB, SeamColor, LineThickness);
					Points->AddPoint(PtA, SeamColor, PointSize);
					Points->AddPoint(PtB, SeamColor, PointSize);
				}
				else
				{
					for (int32 StitchID = 0; StitchID < ConnectedSeam.Num(); ++StitchID)
					{
						const int32 VertexA = ConnectedSeam[StitchID][0];
						const int32 VertexB = ConnectedSeam[StitchID][1];
						const FVector PtA = Mesh.GetVertex(VertexA);
						const FVector PtB = Mesh.GetVertex(VertexB);
						Lines->AddLine(PtA, PtB, SeamColor, 2.0f);
						Points->AddPoint(PtA, SeamColor, PointSize);
						Points->AddPoint(PtB, SeamColor, PointSize);
					}
				}
			}
		}
		else if (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D)
		{
			const TArray<int32> SeamStitches(SeamFacade.GetSeamStitch3DIndex());
			const FColor SeamColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(SeamIndex).ToFColor(true);

			// In 3D we should be able to draw the seam edges in any order, doesn't need to be in connected paths
			for (int32 StitchIndexI = 0; StitchIndexI < SeamStitches.Num(); ++StitchIndexI)
			{
				const int32 VertexIndexI = SeamStitches[StitchIndexI];
				for (int32 StitchIndexJ = StitchIndexI + 1; StitchIndexJ < SeamStitches.Num(); ++StitchIndexJ)
				{
					const int32 VertexIndexJ = SeamStitches[StitchIndexJ];

					if (Mesh.FindEdge(VertexIndexI, VertexIndexJ) != UE::Geometry::FDynamicMesh3::InvalidID)
					{
						const FVector PointI = Mesh.GetVertex(VertexIndexI);
						const FVector PointJ = Mesh.GetVertex(VertexIndexJ);
						Lines->AddLine(PointI, PointJ, SeamColor, LineThickness);
						Points->AddPoint(PointI, SeamColor, PointSize);
						Points->AddPoint(PointJ, SeamColor, PointSize);
					}
				}
			}

		}
	}
}


void UChaosClothAssetEditorMode::InitializeSurfaceNormalDraw()
{
	if (!SurfaceNormalDraw)
	{
		return;
	}

	SurfaceNormalDraw->RemoveAllLineSets();

	if (!bConstructionViewNormalsVisible)
	{
		return;
	}

	if (!DynamicMeshComponent || !DynamicMeshComponent->GetMesh())
	{
		return;
	}

	const UE::Geometry::FDynamicMesh3& Mesh = *DynamicMeshComponent->GetMesh();

	const float LineLength = 2.0f;
	const float LineThickness = 1.0f;

	if (Mesh.HasAttributes())
	{
		if (const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = Mesh.Attributes()->PrimaryNormals())
		{
			SurfaceNormalDraw->CreateOrUpdateLineSet(TEXT("Normals"), NormalOverlay->MaxElementID(),
			[&NormalOverlay, &Mesh, LineLength, LineThickness](int32 Index, TArray<FRenderableLine>& Lines)
			{
				if (NormalOverlay->IsElement(Index))
				{
					int32 ParentVtx = NormalOverlay->GetParentVertex(Index);
					FVector3f Normal = NormalOverlay->GetElement(Index);
					FVector3f Origin = (FVector3f)Mesh.GetVertex(ParentVtx);
					Lines.Add(FRenderableLine((FVector)Origin, (FVector)Origin + LineLength * (FVector)Normal, FColor(15, 15, 240), LineThickness));
				}
			}, 1);
		}
	}
}


void UChaosClothAssetEditorMode::ReinitializeDynamicMeshComponents()
{
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Geometry;

	auto SetUpDynamicMeshComponentMaterial = [this](const UE::Chaos::ClothAsset::FCollectionClothConstFacade& ClothFacade, UDynamicMeshComponent& MeshComponent)
	{
		switch (ConstructionViewMode)
		{
			case EClothPatternVertexType::Sim2D:
			{
				if (bPatternColors)
				{
					constexpr bool bTwoSided = true;
					UMaterialInterface* const Material = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), bTwoSided);
					MeshComponent.SetMaterial(0, Material);
				}
				else
				{
					UMaterialInterface* const Material = ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(GetToolManager(), FLinearColor{ 0.6f, 0.6f, 0.6f }, 0.0);
					MeshComponent.SetMaterial(0, Material);
				}
			}
			break;
			case EClothPatternVertexType::Sim3D:
			{
				if (bPatternColors)
				{
					constexpr bool bTwoSided = true;
					UMaterialInterface* const Material = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), bTwoSided);
					MeshComponent.SetMaterial(0, Material);
				}
				else
				{
					UMaterialInterface* const Material = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
					MeshComponent.SetMaterial(0, Material);
				}
			}
			break;
			case EClothPatternVertexType::Render:
			{
				const TArrayView<const FSoftObjectPath> MaterialPaths = ClothFacade.GetRenderMaterialSoftObjectPathName();
				for (int32 MaterialIndex = 0; MaterialIndex < MaterialPaths.Num(); ++MaterialIndex)
				{
					const FSoftObjectPath& Path = MaterialPaths[MaterialIndex];
					UMaterialInterface* const Material = Cast<UMaterialInterface>(Path.TryLoad());
					MeshComponent.SetMaterial(MaterialIndex, Material);
				}

				// Fix up any triangles without valid material IDs
				int32 DefaultMaterialID = INDEX_NONE;
				for (const int32 TriID : MeshComponent.GetMesh()->TriangleIndicesItr())
				{
					const int32 MaterialID = MeshComponent.GetMesh()->Attributes()->GetMaterialID()->GetValue(TriID);
					if (!MeshComponent.GetMaterial(MaterialID))
					{
						if (DefaultMaterialID == INDEX_NONE)
						{
							DefaultMaterialID = MeshComponent.GetNumMaterials();
							MeshComponent.SetMaterial(DefaultMaterialID, UMaterial::GetDefaultMaterial(MD_Surface));
						}
						MeshComponent.GetMesh()->Attributes()->GetMaterialID()->SetValue(TriID, DefaultMaterialID);
					}
				}

			}
			break;
		}
	};
	
	// Clean up existing DynamicMeshComponent
	// Save indices of selected mesh components

	USelection* SelectedComponents = GetModeManager()->GetSelectedComponents();

	if (DynamicMeshComponent)
	{
		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->SelectionOverrideDelegate.Unbind();

		if (SelectedComponents->IsSelected(DynamicMeshComponent))
		{
			SelectedComponents->Deselect(DynamicMeshComponent);
			DynamicMeshComponent->PushSelectionToProxy();
		}
	}

	if (WireframeDraw)
	{
		WireframeDraw->Disconnect();
	}

	if (ClothSeamDraw)
	{
		ClothSeamDraw->Disconnect();
	}

	if (SurfaceNormalDraw)
	{
		SurfaceNormalDraw->Disconnect();
	}

	PropertyObjectsToTick.Empty();	// TODO: We only want to empty the wireframe display properties. Is anything else using this array?
	DynamicMeshComponent = nullptr;
	DynamicMeshComponentParentActor = nullptr;
	WireframeDraw = nullptr;
	ClothSeamDraw = nullptr;
	SurfaceNormalDraw = nullptr;

	TSharedPtr<FManagedArrayCollection> Collection = bDynamicMeshUseInputCollection ? GetInputClothCollection() : GetClothCollection();
	if (!Collection)
	{
		return;
	}

	const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection.ToSharedRef());

	UE::Geometry::FDynamicMesh3 LodMesh;
	LodMesh.EnableAttributes();
	FClothPatternToDynamicMesh Converter;
	Converter.Convert(Collection.ToSharedRef(), INDEX_NONE, ConstructionViewMode, LodMesh);

	if (ConstructionViewMode == EClothPatternVertexType::Sim2D)
	{
		// Use per-triangle normals for the 2D view
		UE::Geometry::FMeshNormals::InitializeMeshToPerTriangleNormals(&LodMesh);
	}

	if (bPatternColors && ConstructionViewMode != EClothPatternVertexType::Render)
	{
		checkf(LodMesh.TriangleCount() == ClothFacade.GetNumSimFaces(), TEXT("Expected to have the same number of faces in LodMesh and ClothCollection sim mesh"));

		LodMesh.Attributes()->EnablePrimaryColors();
		LodMesh.Attributes()->PrimaryColors()->CreateFromPredicate([&ClothFacade](int ParentVID, int TriIDA, int TriIDB)
			{
				return ClothFacade.FindSimPatternByFaceIndex(TriIDA) == ClothFacade.FindSimPatternByFaceIndex(TriIDB);
			}
		, 0.0f);

		FDynamicMeshColorOverlay* const ColorAttributeLayer = LodMesh.Attributes()->PrimaryColors();
		for (int32 PatternID = 0; PatternID < ClothFacade.GetNumSimPatterns(); ++PatternID)
		{
			const FCollectionClothSimPatternConstFacade Pattern = ClothFacade.GetSimPattern(PatternID);
			const FLinearColor PatternColor = UE::Chaos::ClothAsset::Private::PseudoRandomColor(PatternID);

			for (int32 TriID = 0; TriID < Pattern.GetNumSimFaces(); ++TriID)
			{
				const int32 GlobalTriID = Pattern.GetSimFacesOffset() + TriID;
				FIndex3i AttrTri;
				if (LodMesh.Attributes()->PrimaryColors()->GetTriangleIfValid(GlobalTriID, AttrTri))
				{
					ColorAttributeLayer->SetElement(AttrTri[0], (FVector4f)PatternColor);
					ColorAttributeLayer->SetElement(AttrTri[1], (FVector4f)PatternColor);
					ColorAttributeLayer->SetElement(AttrTri[2], (FVector4f)PatternColor);
				}
			}
		}
	}

	

	// We only need an actor to allow use of HHitProxy for selection
	const FRotator Rotation(0.0f, 0.0f, 0.0f);
	const FActorSpawnParameters SpawnInfo;
	DynamicMeshComponentParentActor = this->GetWorld()->SpawnActor<AActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(DynamicMeshComponentParentActor);
	DynamicMeshComponent->SetMesh(MoveTemp(LodMesh));

	SetUpDynamicMeshComponentMaterial(ClothFacade, *DynamicMeshComponent);

	DynamicMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateUObject(this, &UChaosClothAssetEditorMode::IsComponentSelected);
	DynamicMeshComponent->RegisterComponentWithWorld(this->GetWorld());

	// Set up the wireframe display of the rest space mesh.
	WireframeDraw = NewObject<UMeshElementsVisualizer>(this);
	WireframeDraw->CreateInWorld(GetWorld(), FTransform::Identity);

	WireframeDraw->Settings->DepthBias = 2.0;
	WireframeDraw->Settings->bAdjustDepthBiasUsingMeshSize = false;
	WireframeDraw->Settings->bShowWireframe = true;
	WireframeDraw->Settings->bShowBorders = true;
	WireframeDraw->Settings->bShowUVSeams = false;
	WireframeDraw->Settings->bShowNormalSeams = false;

	// These are not exposed at the visualizer level yet
	// TODO: Should they be?
	WireframeDraw->WireframeComponent->BoundaryEdgeThickness = 2;

	WireframeDraw->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc)
	{
		ProcessFunc(*DynamicMeshComponent->GetMesh());
	});

	DynamicMeshComponent->OnMeshChanged.Add(
		FSimpleMulticastDelegate::FDelegate::CreateLambda([this]()
		{
			WireframeDraw->NotifyMeshChanged();
		}));

	// The settings object and wireframe are not part of a tool, so they won't get ticked like they
	// are supposed to (to enable property watching), unless we add this here.
	PropertyObjectsToTick.Add(WireframeDraw->Settings);

	const bool bRestSpaceMeshVisible = DynamicMeshComponent->GetVisibleFlag();
	WireframeDraw->Settings->bVisible = bRestSpaceMeshVisible && bConstructionViewWireframe;


	ClothSeamDraw = NewObject<UPreviewGeometry>(this);
	ClothSeamDraw->CreateInWorld(GetWorld(), FTransform::Identity);
	InitializeSeamDraw();
	ClothSeamDraw->SetAllVisible(bRestSpaceMeshVisible && bConstructionViewSeamsVisible);

	SurfaceNormalDraw = NewObject<UPreviewGeometry>(this);
	SurfaceNormalDraw->CreateInWorld(GetWorld(), FTransform::Identity);
	InitializeSurfaceNormalDraw();
	SurfaceNormalDraw->SetAllVisible(bRestSpaceMeshVisible && bConstructionViewNormalsVisible);

	DynamicMeshComponent->OnMeshChanged.Add(
		FSimpleMulticastDelegate::FDelegate::CreateLambda([this]()
		{
			InitializeSeamDraw();
			const bool bRestSpaceMeshVisible = DynamicMeshComponent->GetVisibleFlag();
			ClothSeamDraw->SetAllVisible(bRestSpaceMeshVisible&& bConstructionViewSeamsVisible);

			InitializeSurfaceNormalDraw();
			SurfaceNormalDraw->SetAllVisible(bRestSpaceMeshVisible && bConstructionViewNormalsVisible);
		}));


	// Some interactive tools will hide the input DynamicMeshComponent and create their own temporary PreviewMesh for visualization. If this
	// occurs, we should also hide the corresponding Wireframe and Seam drawing (and un-hide it when the tool finishes).
	UActorComponent::MarkRenderStateDirtyEvent.AddWeakLambda(this, [this](UActorComponent& ActorComponent)
		{
			if (!DynamicMeshComponent)
			{
				return;
			}
			const bool bRestSpaceMeshVisible = DynamicMeshComponent->GetVisibleFlag();
			if (WireframeDraw)
			{
				WireframeDraw->Settings->bVisible = bRestSpaceMeshVisible && bConstructionViewWireframe;
			}
			if (ClothSeamDraw)
			{
				ClothSeamDraw->SetAllVisible(bRestSpaceMeshVisible && bConstructionViewSeamsVisible);
			}
			if (SurfaceNormalDraw)
			{
				SurfaceNormalDraw->SetAllVisible(bRestSpaceMeshVisible && bConstructionViewNormalsVisible);
			}
		});


	SelectedComponents->DeselectAll();
	SelectedComponents->Select(DynamicMeshComponent);
	DynamicMeshComponent->PushSelectionToProxy();

	// Update the context object with the ConstructionViewMode and Collection used to build the DynamicMeshComponents, so 
	// tools know how to use the components.
	UpdateContextObject(Collection);
}

void UChaosClothAssetEditorMode::RefocusRestSpaceViewportClient()
{
	TSharedPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient, ESPMode::ThreadSafe> PinnedVC = RestSpaceViewportClient.Pin();
	if (PinnedVC.IsValid())
	{
		// This will happen in FocusViewportOnBox anyways; do it now to get a consistent end result
		PinnedVC->ToggleOrbitCamera(false);

		const FBox SceneBounds = SceneBoundingBox();
		const bool bPattern2DMode = (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D);
		if (bPattern2DMode)
		{
			// 2D pattern
			PinnedVC->SetInitialViewTransform(ELevelViewportType::LVT_Perspective, FVector(0, 0, -100), FRotator(90, -90, 0), DEFAULT_ORTHOZOOM);
		}
		else
		{
			// 3D rest space
			PinnedVC->SetInitialViewTransform(ELevelViewportType::LVT_Perspective, FVector(0, 150, 200), FRotator(0, 0, 0), DEFAULT_ORTHOZOOM);
		}

		constexpr bool bInstant = true;
		PinnedVC->FocusViewportOnBox(SceneBounds, bInstant);

		// Recompute near/far clip planes
		PinnedVC->SetConstructionViewMode(ConstructionViewMode);
	}
}

void UChaosClothAssetEditorMode::FirstTimeFocusRestSpaceViewport()
{
	// If this is the first time seeing a valid 2D or 3D mesh, refocus the camera on it.
	const bool bIsValid = (SelectedClothCollection && DynamicMeshComponent && DynamicMeshComponent->GetMesh()->TriangleCount() > 0);
	const bool bIs2D = ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D;

	if (bIsValid)
	{
		if (bIs2D && bFirstValid2DMesh)
		{
			bFirstValid2DMesh = false;
			RefocusRestSpaceViewportClient();
		}
		else if (!bIs2D && bFirstValid3DMesh)
		{
			bFirstValid3DMesh = false;
			RefocusRestSpaceViewportClient();
		}
	}
}

void UChaosClothAssetEditorMode::InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn)
{
	// InitializeContexts needs to have been called first so that we have the 3d preview world ready.
	check(PreviewScene);

	UBaseCharacterFXEditorMode::InitializeTargets(AssetsIn);
}

void UChaosClothAssetEditorMode::SoftResetSimulation()
{
	bShouldResetSimulation = true;
	bShouldClearTeleportFlag = false;
	bHardReset = false;
}

void UChaosClothAssetEditorMode::HardResetSimulation()
{
	bShouldResetSimulation = true;
	bShouldClearTeleportFlag = false;
	bHardReset = true;
}

void UChaosClothAssetEditorMode::SuspendSimulation()
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		PreviewScene->GetClothComponent()->SuspendSimulation();
	}
}

void UChaosClothAssetEditorMode::ResumeSimulation()
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		PreviewScene->GetClothComponent()->ResumeSimulation();
	}
}

bool UChaosClothAssetEditorMode::IsSimulationSuspended() const
{
	if (PreviewScene && PreviewScene->GetClothComponent() && PreviewScene->GetClothComponent()->GetClothSimulationProxy())
	{
		return PreviewScene->GetClothComponent()->IsSimulationSuspended();
	}

	return false;
}

void UChaosClothAssetEditorMode::SetEnableSimulation(bool bEnable)
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		PreviewScene->GetClothComponent()->SetEnableSimulation(bEnable);
	}
}

bool UChaosClothAssetEditorMode::IsSimulationEnabled() const
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		return PreviewScene->GetClothComponent()->IsSimulationEnabled();
	}

	return false;
}

int32 UChaosClothAssetEditorMode::GetConstructionViewTriangleCount() const
{
	if (DynamicMeshComponent && DynamicMeshComponent->GetMesh())
	{
		return DynamicMeshComponent->GetMesh()->TriangleCount();
	}
	return 0;
}

int32 UChaosClothAssetEditorMode::GetConstructionViewVertexCount() const
{
	if (DynamicMeshComponent && DynamicMeshComponent->GetMesh())
	{
		return DynamicMeshComponent->GetMesh()->VertexCount();
	}
	return 0;
}

void UChaosClothAssetEditorMode::SetLODModel(int32 LODIndex)
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		PreviewScene->GetClothComponent()->SetForcedLOD(LODIndex + 1);
	}
}

bool UChaosClothAssetEditorMode::IsLODModelSelected(int32 LODIndex) const
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		return PreviewScene->GetClothComponent()->GetForcedLOD() == LODIndex + 1;
	}
	return false;
}

int32 UChaosClothAssetEditorMode::GetLODModel() const
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		return PreviewScene->GetClothComponent()->GetForcedLOD() - 1;
	}
	return INDEX_NONE;
}

int32 UChaosClothAssetEditorMode::GetNumLODs() const
{
	if (PreviewScene && PreviewScene->GetClothComponent())
	{
		return PreviewScene->GetClothComponent()->GetNumLODs();
	}
	return 0;
}

void UChaosClothAssetEditorMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);
	

	if (TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		// For now don't allow selection change once the tool has uncommitted changes
		// TODO: We might want to auto-accept unsaved changes and allow switching between nodes
		if (GetInteractiveToolsContext()->CanAcceptActiveTool())
		{
			GraphEditor->SetEnabled(false);
		}
		else
		{
			GraphEditor->SetEnabled(true);
		}
	}


	for (TObjectPtr<UInteractiveToolPropertySet>& Propset : PropertyObjectsToTick)
	{
		if (Propset)
		{
			if (Propset->IsPropertySetEnabled())
			{
				Propset->CheckAndUpdateWatched();
			}
			else
			{
				Propset->SilentUpdateWatched();
			}
		}
	}

	if (WireframeDraw)
	{
		WireframeDraw->OnTick(DeltaTime);
	}


	if (bShouldClearTeleportFlag)
	{
		PreviewScene->GetClothComponent()->ResetTeleportMode();
		bShouldClearTeleportFlag = false;
	}

	if (bShouldResetSimulation)
	{
		if (bHardReset)
		{
			const FComponentReregisterContext Context(PreviewScene->GetClothComponent());
		}
		else
		{
			PreviewScene->GetClothComponent()->ForceNextUpdateTeleportAndReset();
		}

		bShouldResetSimulation = false;
		bShouldClearTeleportFlag = true;		// clear the flag next tick
	}

	if (bDynamicMeshComponentInitDeferred)
	{
		ReinitializeDynamicMeshComponents();

		// The first time we get a valid mesh, refocus the camera on it
		FirstTimeFocusRestSpaceViewport();

		bDynamicMeshComponentInitDeferred = false;
	}

	if (!NodeTypeForPendingToolStart.IsNone() && !GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		const TSharedRef<FUICommandList> CommandList = Toolkit->GetToolkitCommands();
		const UE::Chaos::ClothAsset::FChaosClothAssetEditorCommands& CommandInfos = UE::Chaos::ClothAsset::FChaosClothAssetEditorCommands::Get();

		if (const TSharedPtr<const FUICommandInfo>* const Command = NodeTypeToToolCommandMap.Find(NodeTypeForPendingToolStart))
		{
			CommandList->TryExecuteAction(Command->ToSharedRef());
		}

		NodeTypeForPendingToolStart = FName();
	}

	const bool bIsInPIEOrSIE = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;
	const bool bShouldPause = PreviewScene->GetPreviewSceneDescription()->bPauseWhilePlayingInEditor && bIsInPIEOrSIE;

	if (!bShouldPause)
	{
		if (PreviewScene->GetWorld())
		{
			PreviewScene->GetWorld()->Tick(ELevelTick::LEVELTICK_All, DeltaTime);
		}
	}
}

void UChaosClothAssetEditorMode::RestSpaceViewportResized(FViewport* RestspaceViewport, uint32 /*Unused*/)
{
	// We'd like to call RefocusRestSpaceViewportClient() when the viewport is first created, however in Ortho mode the
	// viewport needs to have non-zero size for FocusViewportOnBox() to work properly. So we wait until the viewport is resized here.
	if (bShouldFocusRestSpaceView && RestspaceViewport && RestspaceViewport->GetSizeXY().X > 0 && RestspaceViewport->GetSizeXY().Y > 0)
	{
		RefocusRestSpaceViewportClient();
		bShouldFocusRestSpaceView = false;
	}
}

FBox UChaosClothAssetEditorMode::SceneBoundingBox() const
{
	FBoxSphereBounds TotalBounds(ForceInitToZero);
	
	if (DynamicMeshComponent)
	{
		TotalBounds = DynamicMeshComponent->Bounds;
	}

	return TotalBounds.GetBox();
}

FBox UChaosClothAssetEditorMode::SelectionBoundingBox() const
{
	// if Tool supports custom Focus box, use that first
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* const Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* const FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusBox())
		{
			return FocusAPI->GetWorldSpaceFocusBox();
		}
	}

	const USelection* const SelectedComponents = GetModeManager()->GetSelectedComponents();

	if (DynamicMeshComponent && SelectedComponents->IsSelected(DynamicMeshComponent))
	{
		return DynamicMeshComponent->Bounds.GetBox();
	}
	
	// Nothing selected, return the whole scene
	return SceneBoundingBox();
}


FBox UChaosClothAssetEditorMode::PreviewBoundingBox() const
{
	FBox Bounds(ForceInit);

	if (const UChaosClothComponent* const Cloth = PreviewScene->GetClothComponent())
	{
		if (Cast<UChaosClothAsset>(Cloth->GetAsset()))
		{
			Bounds += Cloth->Bounds.GetBox();
		}
	}

	if (const USkeletalMeshComponent* const SkeletalMesh = PreviewScene->GetSkeletalMeshComponent())
	{
		if (SkeletalMesh->GetSkeletalMeshAsset())
		{
			Bounds += SkeletalMesh->Bounds.GetBox();
		}
	}

	return Bounds;
}

void UChaosClothAssetEditorMode::SetConstructionViewMode(UE::Chaos::ClothAsset::EClothPatternVertexType InMode)
{
	// We will first check if there is an active tool. If so, we'll shut down the tool and save the results to the Node, then change view modes, then restart the tool again.
	bool bEndedActiveTool = false;
	UInteractiveToolManager* const ToolManager = GetInteractiveToolsContext()->ToolManager;
	checkf(ToolManager, TEXT("No valid ToolManager found for UChaosClothAssetEditorMode"));
	if (UInteractiveTool* const ActiveTool = ToolManager->GetActiveTool(EToolSide::Left))
	{
		// avoid switching back to the previous view mode when the tool ends here
		const bool bTempShouldRestoreVal = bShouldRestoreSavedConstructionViewMode;
		bShouldRestoreSavedConstructionViewMode = false;

		ToolManager->PostActiveToolShutdownRequest(ActiveTool, EToolShutdownType::Accept);
		bEndedActiveTool = true;

		// now we can restore the previous view mode the next time the tool ends
		bShouldRestoreSavedConstructionViewMode = bTempShouldRestoreVal;
	}

	ConstructionViewMode = InMode;
	ReinitializeDynamicMeshComponents();

	const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient> VC = RestSpaceViewportClient.Pin();
	if (VC.IsValid())
	{
		VC->SetConstructionViewMode(ConstructionViewMode);
	}

	// If we are switching to a mode with a valid mesh for the first time, focus the camera on it
	FirstTimeFocusRestSpaceViewport();

	if (bEndedActiveTool)
	{
		// If we ended the active tool in order to change modes, restart it now
		if (const TSharedPtr<const SDataflowGraphEditor> PinnedGraphEditor = DataflowGraphEditor.Pin())
		{
			const FGraphPanelSelectionSet& SelectedNodes = PinnedGraphEditor->GetSelectedNodes();
			if (SelectedNodes.Num() == 1)
			{
				StartToolForSelectedNode(*SelectedNodes.CreateConstIterator());
			}
		}
	}

}

UE::Chaos::ClothAsset::EClothPatternVertexType UChaosClothAssetEditorMode::GetConstructionViewMode() const
{
	return ConstructionViewMode;
}

bool UChaosClothAssetEditorMode::CanChangeConstructionViewModeTo(UE::Chaos::ClothAsset::EClothPatternVertexType NewViewMode) const
{
	if (!GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return true;
	}

	const UInteractiveToolBuilder* const ActiveToolBuilder = GetToolManager()->GetActiveToolBuilder(EToolSide::Left);
	checkf(ActiveToolBuilder, TEXT("No Active Tool Builder found despite having an Active Tool"));

	const IChaosClothAssetEditorToolBuilder* const ClothToolBuilder = Cast<const IChaosClothAssetEditorToolBuilder>(ActiveToolBuilder);
	checkf(ClothToolBuilder, TEXT("Cloth Editor has an active Tool Builder that does not implement IChaosClothAssetEditorToolBuilder"));

	const UEditorInteractiveToolsContext* const RestSpaceToolsContext = GetInteractiveToolsContext();
	checkf(RestSpaceToolsContext, TEXT("Cloth Editor Mode doesn't have a valid InteractiveToolsContext"));

	const UDataflowContextObject* const EditorContextObject = RestSpaceToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
	checkf(EditorContextObject, TEXT("ContextObject not found in ContextObjectStore despite having an active tool. This should have been created by the time a tool is activated"));

	TArray<UE::Chaos::ClothAsset::EClothPatternVertexType> SupportedViewModes;
	ClothToolBuilder->GetSupportedViewModes(*EditorContextObject, SupportedViewModes);

	return SupportedViewModes.Contains(NewViewMode);
}

void UChaosClothAssetEditorMode::ToggleConstructionViewWireframe()
{
	bConstructionViewWireframe = !bConstructionViewWireframe;
	ReinitializeDynamicMeshComponents();
}

bool UChaosClothAssetEditorMode::CanSetConstructionViewWireframeActive() const
{
	if (!GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return true;
	}

	const UInteractiveToolBuilder* const ActiveToolBuilder = GetToolManager()->GetActiveToolBuilder(EToolSide::Left);
	checkf(ActiveToolBuilder, TEXT("No Active Tool Builder found despite having an Active Tool"));

	const IChaosClothAssetEditorToolBuilder* const ClothToolBuilder = Cast<const IChaosClothAssetEditorToolBuilder>(ActiveToolBuilder);
	checkf(ClothToolBuilder, TEXT("Cloth Editor has an active Tool Builder that does not implement IChaosClothAssetEditorToolBuilder"));
	return ClothToolBuilder->CanSetConstructionViewWireframeActive();
}


void UChaosClothAssetEditorMode::ToggleConstructionViewSeams()
{
	bConstructionViewSeamsVisible = !bConstructionViewSeamsVisible;
	ReinitializeDynamicMeshComponents();
}

bool UChaosClothAssetEditorMode::CanSetConstructionViewSeamsActive() const
{
	// Disallow seam view when any tool is active
	if (GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	// Seam view not available on non-manifold meshes
	if (DynamicMeshComponent)
	{
		if (const UE::Geometry::FDynamicMesh3* const Mesh = DynamicMeshComponent->GetMesh())
		{
			const UE::Geometry::FNonManifoldMappingSupport NonManifold(*Mesh);
			if (NonManifold.IsNonManifoldVertexInSource())
			{
				return false;
			}
		}
	}

	return true;
}


void UChaosClothAssetEditorMode::ToggleConstructionViewSeamsCollapse()
{
	bConstructionViewSeamsCollapse = !bConstructionViewSeamsCollapse;
	ReinitializeDynamicMeshComponents();
}

bool UChaosClothAssetEditorMode::CanSetConstructionViewSeamsCollapse() const
{
	// Disallow seam view when any tool is active
	if (GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	return bConstructionViewSeamsVisible && (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D);
}

void UChaosClothAssetEditorMode::ToggleConstructionViewSurfaceNormals()
{
	bConstructionViewNormalsVisible = !bConstructionViewNormalsVisible;
	ReinitializeDynamicMeshComponents();
}

bool UChaosClothAssetEditorMode::CanSetConstructionViewSurfaceNormalsActive() const
{
	if (GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	return true;
}


void UChaosClothAssetEditorMode::TogglePatternColor()
{
	bPatternColors = !bPatternColors;
	ReinitializeDynamicMeshComponents();
}

bool UChaosClothAssetEditorMode::CanSetPatternColor() const
{	
	// Disallow pattern color view when any tool is active
	if (GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	return (ConstructionViewMode != UE::Chaos::ClothAsset::EClothPatternVertexType::Render);
}

void UChaosClothAssetEditorMode::ToggleMeshStats()
{
	bMeshStats = !bMeshStats;
}

bool UChaosClothAssetEditorMode::CanSetMeshStats() const
{
	// Disallow seam view when any tool is active
	if (GetToolManager()->HasActiveTool(EToolSide::Left))
	{
		return false;
	}

	return true;
}

void UChaosClothAssetEditorMode::SetRestSpaceViewportClient(TWeakPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient, ESPMode::ThreadSafe> InViewportClient)
{
	RestSpaceViewportClient = InViewportClient;

	TSharedPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient> VC = RestSpaceViewportClient.Pin();
	if (VC.IsValid())
	{
		VC->SetConstructionViewMode(ConstructionViewMode);
		VC->SetToolCommandList(ToolCommandList);

		if (VC->Viewport)
		{
			VC->Viewport->ViewportResizedEvent.AddUObject(this, &UChaosClothAssetEditorMode::RestSpaceViewportResized);
		}
	}
}


void UChaosClothAssetEditorMode::InitializeContextObject()
{
	UEditorInteractiveToolsContext* const RestSpaceToolsContext = GetInteractiveToolsContext();

	// Dataflow context object
	UDataflowContextObject* DataflowContextObject = RestSpaceToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>();
	if (!DataflowContextObject)
	{
		DataflowContextObject = NewObject<UDataflowContextObject>();
		RestSpaceToolsContext->ContextObjectStore->AddContextObject(DataflowContextObject);
	}

	DataflowContextObject->SetConstructionViewMode(UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(ClothViewModeToDataflowViewModeName(ConstructionViewMode)));

	if (const TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
	{
		if (UEdGraphNode* const SingleSelectedNode = GraphEditor->GetSingleSelectedNode())
		{
			if (UDataflowEdNode* const SelectedDataflowEdNode = Cast<UDataflowEdNode>(SingleSelectedNode))
			{
				DataflowContextObject->SetSelectedNode(SelectedDataflowEdNode);
			}
		}
	}
	DataflowContextObject->SetDataflowContext(DataflowContext.Pin());
	DataflowContextObject->SetDataflowAsset(DataflowGraph.Get());
}

void UChaosClothAssetEditorMode::UpdateContextObject(const TSharedPtr<FManagedArrayCollection>& Collection)
{
	UEditorInteractiveToolsContext* const RestSpaceToolsContext = GetInteractiveToolsContext();
	
	if (UDataflowContextObject* DataflowContextObject = RestSpaceToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
	{
		DataflowContextObject->SetConstructionViewMode(UE::Dataflow::FRenderingViewModeFactory::GetInstance().GetViewMode(ClothViewModeToDataflowViewModeName(ConstructionViewMode)));

		DataflowContextObject->SetSelectedCollection(Collection, bDynamicMeshUseInputCollection);
	}
}

void UChaosClothAssetEditorMode::DeleteContextObject()
{
	UEditorInteractiveToolsContext* const RestSpaceToolsContext = GetInteractiveToolsContext();
	if (UDataflowContextObject* DataflowContextObject = RestSpaceToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
	{
		RestSpaceToolsContext->ContextObjectStore->RemoveContextObject(DataflowContextObject);
	}
}

void UChaosClothAssetEditorMode::SetDataflowContext(TWeakPtr<UE::Dataflow::FEngineContext> InDataflowContext)
{
	DataflowContext = InDataflowContext;
	UEditorInteractiveToolsContext* const RestSpaceToolsContext = GetInteractiveToolsContext();
	if (UDataflowContextObject* DataflowContextObject = RestSpaceToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
	{
		DataflowContextObject->SetDataflowContext(DataflowContext.Pin());
	}
}

void UChaosClothAssetEditorMode::SetDataflowGraphEditor(TSharedPtr<SDataflowGraphEditor> InGraphEditor)
{
	DataflowGraphEditor = InGraphEditor;
	if (InGraphEditor)
	{
		InitializeContextObject();
	}
	else
	{
		DeleteContextObject();
	}
}

void UChaosClothAssetEditorMode::StartToolForSelectedNode(const UObject* SelectedNode)
{
	if (const UDataflowEdNode* const EdNode = Cast<UDataflowEdNode>(SelectedNode))
	{
		if (const TSharedPtr<const FDataflowNode> DataflowNode = EdNode->GetDataflowNode())
		{
			if (!DataflowNode->IsFrozen())
			{
				const FName DataflowNodeType = DataflowNode->GetType();
				NodeTypeForPendingToolStart = DataflowNodeType;
			}
		}
	}
}


void UChaosClothAssetEditorMode::OnDataflowNodeDeleted(const TSet<UObject*>& DeletedNodes)
{
	UEditorInteractiveToolsContext* const ToolsContext = GetInteractiveToolsContext();
	checkf(ToolsContext, TEXT("No valid ToolsContext found for UChaosClothAssetEditorMode"));
	const bool bCanCancel = ToolsContext->CanCancelActiveTool();
	ToolsContext->EndTool(bCanCancel ? EToolShutdownType::Cancel : EToolShutdownType::Completed);
}

UEdGraphNode* UChaosClothAssetEditorMode::GetSingleSelectedNodeWithOutputType(const FName& SelectedNodeOutputTypeName) const
{
	const TSharedPtr<const SDataflowGraphEditor> PinnedDataflowGraphEditor = DataflowGraphEditor.Pin();
	if (!PinnedDataflowGraphEditor)
	{
		return nullptr;
	}

	UEdGraphNode* const SelectedNode = PinnedDataflowGraphEditor->GetSingleSelectedNode();
	if (!SelectedNode)
	{
		return nullptr;
	}

	const UDataflowEdNode* const SelectedDataflowEdNode = Cast<UDataflowEdNode>(SelectedNode);
	if (!SelectedDataflowEdNode)
	{
		// The graph can contain UEdGraphNode_Comment, which is not a UDataflowEdNode
		return nullptr;
	}

	const TSharedPtr<const FDataflowNode> SelectedDataflowNode = SelectedDataflowEdNode->GetDataflowNode();

	if (!SelectedDataflowNode)
	{
		// This can happen when the user deletes a node. Seems like the Dataflow FGraph is updated with the removed node before the graph editor can update.
		return nullptr;
	}

	for (const FDataflowOutput* const Output : SelectedDataflowNode->GetOutputs())
	{
		if (Output->GetType() == SelectedNodeOutputTypeName)
		{
			return SelectedNode;
		}
	}

	return nullptr;
}

UEdGraphNode* UChaosClothAssetEditorMode::CreateNewNode(const FName& NewNodeTypeName)
{
	const TSharedPtr<const SDataflowGraphEditor> PinnedDataflowGraphEditor = DataflowGraphEditor.Pin();
	if (!PinnedDataflowGraphEditor)
	{
		return nullptr;
	}

	checkf(DataflowGraph.IsValid(), TEXT("Dataflow pointer is invalid in UChaosClothAssetEditorMode"));

	const TSharedPtr<FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode> NodeAction =
		FAssetSchemaAction_Dataflow_CreateNode_DataflowEdNode::CreateAction(DataflowGraph.Get(), NewNodeTypeName);
	constexpr UEdGraphPin* FromPin = nullptr;
	constexpr bool bSelectNewNode = true;
	UEdGraphNode* const NewEdNode = NodeAction->PerformAction(DataflowGraph.Get(), FromPin, PinnedDataflowGraphEditor->GetPasteLocation2f(), bSelectNewNode);

	return NewEdNode;
}


UEdGraphNode* UChaosClothAssetEditorMode::CreateAndConnectNewNode(const FName& NewNodeTypeName, UEdGraphNode& UpstreamNode, const FName& ConnectionTypeName, const FName& NewNodeConnectionName)
{
	checkf(DataflowGraph.IsValid(), TEXT("Dataflow pointer is invalid in UChaosClothAssetEditorMode"));

	// First find the specified output of the upstream node, plus any pins it's connected to

	UEdGraphPin* UpstreamNodeOutputPin = nullptr;
	TArray<UEdGraphPin*> ExistingNodeInputPins;

	const UDataflowEdNode* const UpstreamDataflowEdNode = CastChecked<UDataflowEdNode>(&UpstreamNode);
	const TSharedPtr<const FDataflowNode> UpstreamDataflowNode = UpstreamDataflowEdNode->GetDataflowNode();

	for (const FDataflowOutput* const Output : UpstreamDataflowNode->GetOutputs())
	{
		if (Output->GetType() == ConnectionTypeName)
		{
			UpstreamNodeOutputPin = UpstreamDataflowEdNode->FindPin(*Output->GetName().ToString(), EGPD_Output);
			ExistingNodeInputPins = UpstreamNodeOutputPin->LinkedTo;
			break;
		}
	}

	// Add the new node 

	UEdGraphNode* const NewEdNode = CreateNewNode(NewNodeTypeName);
	checkf(NewEdNode, TEXT("Failed to create a new node in the DataflowGraph"));

	UDataflowEdNode* const NewDataflowEdNode = CastChecked<UDataflowEdNode>(NewEdNode);
	const TSharedPtr<FDataflowNode> NewDataflowNode = NewDataflowEdNode->GetDataflowNode();

	// Re-wire the graph

	if (UpstreamNodeOutputPin)
	{
		UEdGraphPin* NewNodeInputPin = nullptr;
		for (const FDataflowInput* const NewNodeInput : NewDataflowNode->GetInputs())
		{
			if (NewNodeInput->GetType() == ConnectionTypeName && NewNodeInput->GetName() == NewNodeConnectionName)
			{
				NewNodeInputPin = NewDataflowEdNode->FindPin(*NewNodeInput->GetName().ToString(), EGPD_Input);
			}
		}

		UEdGraphPin* NewNodeOutputPin = nullptr;
		for (const FDataflowOutput* const NewNodeOutput : NewDataflowNode->GetOutputs())
		{
			if (NewNodeOutput->GetType() == ConnectionTypeName && NewNodeOutput->GetName() == NewNodeConnectionName)
			{
				NewNodeOutputPin = NewDataflowEdNode->FindPin(*NewNodeOutput->GetName().ToString(), EGPD_Output);
				break;
			}
		}

		check(NewNodeInputPin);
		check(NewNodeOutputPin);

		DataflowGraph->GetSchema()->TryCreateConnection(UpstreamNodeOutputPin, NewNodeInputPin);

		for (UEdGraphPin* DownstreamInputPin : ExistingNodeInputPins)
		{
			DataflowGraph->GetSchema()->TryCreateConnection(NewNodeOutputPin, DownstreamInputPin);
		}
	}

	DataflowGraph->NotifyGraphChanged();

	return NewEdNode;
}

#undef LOCTEXT_NAMESPACE

