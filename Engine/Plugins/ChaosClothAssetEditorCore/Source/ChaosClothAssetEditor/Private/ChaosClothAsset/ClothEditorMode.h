// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorMode.h"
#include "GeometryBase.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Delegates/IDelegateInstance.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"
#include "ClothEditorMode.generated.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);

class FEditorViewportClient;
class FAssetEditorModeManager;
class FToolCommandChange;
class UMeshElementsVisualizer;
class UPreviewMesh;
class UToolTarget;
class FToolTargetTypeRequirements;
class UWorld;
class UInteractiveToolPropertySet; 
class UMeshOpPreviewWithBackgroundCompute;
class UClothToolViewportButtonsAPI;
class UDynamicMeshComponent;
class UChaosClothComponent;
class FEditorViewportClient;
class FViewport;
class UDataflow;
class SDataflowGraphEditor;
struct FManagedArrayCollection;

namespace UE::Chaos::ClothAsset
{
class FChaosClothPreviewScene;
class FChaosClothAssetEditorModeToolkit;
class FChaosClothAssetEditorToolkit;
class FChaosClothEditorRestSpaceViewportClient;
}
class IChaosClothAssetEditorToolBuilder;
class UEdGraphNode;
class UPreviewGeometry;

/**
 * The cloth editor mode is the mode used in the cloth asset editor. It holds most of the inter-tool state.
 * We put things in a mode instead of directly into the asset editor in case we want to someday use the mode
 * in multiple asset editors.
 */
UCLASS(MinimalAPI, Transient)
class UChaosClothAssetEditorMode final : public UBaseCharacterFXEditorMode
{
	GENERATED_BODY()

public:

	UE_API const static FEditorModeID EM_ChaosClothAssetEditorModeId;

	UE_API UChaosClothAssetEditorMode();

	// Bounding box for selected rest space mesh components
	UE_API FBox SelectionBoundingBox() const;

	// Bounding box for sim space meshes
	UE_API FBox PreviewBoundingBox() const;

	UE_API void SetConstructionViewMode(UE::Chaos::ClothAsset::EClothPatternVertexType InMode);
	UE_API UE::Chaos::ClothAsset::EClothPatternVertexType GetConstructionViewMode() const;
	UE_API bool CanChangeConstructionViewModeTo(UE::Chaos::ClothAsset::EClothPatternVertexType NewViewMode) const;

	UE_API void ToggleConstructionViewWireframe();
	UE_API bool CanSetConstructionViewWireframeActive() const;
	bool IsConstructionViewWireframeActive() const
	{
		return bConstructionViewWireframe;
	}

	UE_API void ToggleConstructionViewSeams();
	UE_API bool CanSetConstructionViewSeamsActive() const;
	bool IsConstructionViewSeamsActive() const
	{
		return bConstructionViewSeamsVisible;
	}

	UE_API void ToggleConstructionViewSeamsCollapse();
	UE_API bool CanSetConstructionViewSeamsCollapse() const;
	bool IsConstructionViewSeamsCollapseActive() const
	{
		return bConstructionViewSeamsCollapse;
	}

	UE_API void TogglePatternColor();
	UE_API bool CanSetPatternColor() const;
	bool IsPatternColorActive() const
	{
		return bPatternColors;
	}

	UE_API void ToggleConstructionViewSurfaceNormals();
	UE_API bool CanSetConstructionViewSurfaceNormalsActive() const;
	bool IsConstructionViewSurfaceNormalsActive() const
	{
		return bConstructionViewNormalsVisible;
	}

	UE_API void ToggleMeshStats();
	UE_API bool CanSetMeshStats() const;
	bool IsMeshStatsActive() const
	{
		return bMeshStats;
	}

	// Simulation controls
	UE_API void SoftResetSimulation();
	UE_API void HardResetSimulation();
	UE_API void SuspendSimulation();
	UE_API void ResumeSimulation();
	UE_API bool IsSimulationSuspended() const;
	UE_API void SetEnableSimulation(bool bEnabled);
	UE_API bool IsSimulationEnabled() const;

	UE_API int32 GetConstructionViewTriangleCount() const;
	UE_API int32 GetConstructionViewVertexCount() const;

	// LODIndex == INDEX_NONE is LOD Auto
	UE_API void SetLODModel(int32 LODIndex);
	UE_API bool IsLODModelSelected(int32 LODIndex) const;
	UE_API int32 GetLODModel() const;
	UE_API int32 GetNumLODs() const;

	TObjectPtr<UEditorInteractiveToolsContext> GetActiveToolsContext()
	{
		return ActiveToolsContext;
	}

private:

	friend class UE::Chaos::ClothAsset::FChaosClothAssetEditorToolkit;
	friend class UE::Chaos::ClothAsset::FChaosClothAssetEditorModeToolkit;

	// UEdMode
	UE_API virtual void Enter() final;
	UE_API virtual void Exit() override;
	UE_API virtual void ModeTick(float DeltaTime) override;
	UE_API virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;
	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void CreateToolkit() override;
	UE_API virtual void BindCommands() override;

	// (We don't actually override MouseEnter, etc, because things get forwarded to the input
	// router via FEditorModeTools, and we don't have any additional input handling to do at the mode level.)

	// UBaseCharacterFXEditorMode
	UE_API virtual void AddToolTargetFactories() override;
	UE_API virtual void RegisterTools() override;
	UE_API virtual void CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;
	UE_API virtual void InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;

	// Gets the tool target requirements for the mode. The resulting targets undergo further processing
	// to turn them into the input objects that tools get (since these need preview meshes, etc).
	static UE_API const FToolTargetTypeRequirements& GetToolTargetRequirements();

	// Use this function to register tools rather than UEdMode::RegisterTool() because we need to specify the ToolsContext
	UE_API void RegisterClothTool(TSharedPtr<FUICommandInfo> UICommand, 
		FString ToolIdentifier, 
		UInteractiveToolBuilder* Builder,
		const IChaosClothAssetEditorToolBuilder* ClothToolBuilder,
		UEditorInteractiveToolsContext* UseToolsContext, 
		EToolsContextScope ToolScope = EToolsContextScope::Default);

	UE_API void RegisterAddNodeCommand(TSharedPtr<FUICommandInfo> AddNodeCommand, const FName& NewNodeType, TSharedPtr<FUICommandInfo> StartToolCommand);

	UE_API void SetPreviewScene(UE::Chaos::ClothAsset::FChaosClothPreviewScene* PreviewScene);

	// Bounding box for rest space meshes
	UE_API virtual FBox SceneBoundingBox() const override;

	UE_API void SetRestSpaceViewportClient(TWeakPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient, ESPMode::ThreadSafe> ViewportClient);
	UE_API void RefocusRestSpaceViewportClient();
	UE_API void FirstTimeFocusRestSpaceViewport();

	// intended to be called by the toolkit when selected node in the Dataflow graph changes
	UE_API void SetSelectedClothCollection(TSharedPtr<FManagedArrayCollection> Collection, TSharedPtr<FManagedArrayCollection> InputCollection = nullptr, bool bDeferDynamicMeshInitForTool = false);

	// gets the currently selected cloth collection, as specified by the toolkit
	UE_API TSharedPtr<FManagedArrayCollection> GetClothCollection();
	UE_API TSharedPtr<FManagedArrayCollection> GetInputClothCollection();

	UE_API void SetDataflowContext(TWeakPtr<UE::Dataflow::FEngineContext> InDataflowContext);
	UE_API void SetDataflowGraphEditor(TSharedPtr<SDataflowGraphEditor> InGraphEditor);
	
	UE_API void StartToolForSelectedNode(const UObject* SelectedNode);
	UE_API void OnDataflowNodeDeleted(const TSet<UObject*>& DeletedNodes);

	/**
	* Return the single selected node in the Dataflow Graph Editor only if it has an output of the specified type
	* If there is not a single node selected, or if it does not have the specified output, return null
	*/
	UE_API UEdGraphNode* GetSingleSelectedNodeWithOutputType(const FName& SelectedNodeOutputTypeName) const;

	/**
	 * Create a node with the specified type in the graph
	*/
	UE_API UEdGraphNode* CreateNewNode(const FName& NewNodeTypeName);

	/** Create a node with the specified type, then connect it to the output of the specified UpstreamNode.
	* If the specified output of the upstream node is already connected to another node downstream, we first break
	* that connecttion, then insert the new node along the previous connection.
	* We want to turn this:
	*
	* [UpstreamNode] ----> [DownstreamNode(s)]
	*
	* to this:
	*
	* [UpstreamNode] ----> [NewNode] ----> [DownstreamNode(s)]
	*
	*
	* @param NewNodeTypeName The type of node to create, by name
	* @param UpstreamNode Node to connect the new node to
	* @param ConnectionTypeName The type of output of the upstream node to connect our new node to
	* @param NewNodeConnectionName The name of the input/output connection on our new node that will be connected
	* @return The newly-created node
	*/
	UE_API UEdGraphNode* CreateAndConnectNewNode(const FName& NewNodeTypeName,	UEdGraphNode& UpstreamNode,	const FName& ConnectionTypeName, const FName& NewNodeConnectionName);

	UE_API void AddNode(FName NewNodeType);
	UE_API bool CanAddNode(FName NewNodeType) const;


	UE_API void InitializeContextObject();
	UE_API void UpdateContextObject(const TSharedPtr<FManagedArrayCollection>& Collection);
	UE_API void DeleteContextObject();

	UE_API bool IsComponentSelected(const UPrimitiveComponent* InComponent);

	// Rest space wireframe. They have to get ticked to be able to respond to setting changes. 
	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> WireframeDraw = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> ClothSeamDraw = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> SurfaceNormalDraw = nullptr;

	// Preview Scene, here largely for convenience to avoid having to pass it around functions. Owned by the ClothEditorToolkit.
	UE::Chaos::ClothAsset::FChaosClothPreviewScene* PreviewScene = nullptr;

	// Mode-level property objects (visible or not) that get ticked.
	UPROPERTY()
	TArray<TObjectPtr<UInteractiveToolPropertySet>> PropertyObjectsToTick;

	// Rest space editable mesh
	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	// Actor required for hit testing DynamicMeshComponent
	UPROPERTY()
	TObjectPtr<AActor> DynamicMeshComponentParentActor = nullptr;

	TWeakPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient, ESPMode::ThreadSafe> RestSpaceViewportClient;

	// The dynamic mesh component needs to be reinitialized on next tick.
	bool bDynamicMeshComponentInitDeferred = false;
	// Use the input collection to build the dynamic mesh component (used by tools).
	bool bDynamicMeshUseInputCollection = false;

	// The first time we get a valid mesh, refocus the camera on it
	bool bFirstValid2DMesh = true;
	bool bFirstValid3DMesh = true;

	// Whether the rest space viewport should focus on the rest space mesh on the next tick
	bool bShouldFocusRestSpaceView = true;

	UE_API void RestSpaceViewportResized(FViewport* RestspaceViewport, uint32 Unused);

	UE::Chaos::ClothAsset::EClothPatternVertexType ConstructionViewMode;

	// The Construction view mode that was active before starting the current tool. When the tool ends, restore this view mode if bShouldRestoreSavedConstructionViewMode is true.
	UE::Chaos::ClothAsset::EClothPatternVertexType SavedConstructionViewMode;

	// Whether we should restore the previous view mode when a tool ends
	bool bShouldRestoreSavedConstructionViewMode = false;

	// Dataflow node type whose corresponding tool should be started on the next Tick
	FName NodeTypeForPendingToolStart;

	bool bConstructionViewWireframe = false;
	bool bShouldRestoreConstructionViewWireframe = false;

	bool bConstructionViewSeamsVisible = false;
	bool bShouldRestoreConstructionViewSeams = false;
	bool bConstructionViewSeamsCollapse = false;
	UE_API void InitializeSeamDraw();

	bool bConstructionViewNormalsVisible = false;
	UE_API void InitializeSurfaceNormalDraw();

	bool bPatternColors = false;
	bool bMeshStats = false;

	// Create dynamic mesh components from the cloth component's rest space info
	UE_API void ReinitializeDynamicMeshComponents();

	// Simulation controls
	bool bShouldResetSimulation = false;
	bool bHardReset = false;
	bool bShouldClearTeleportFlag = false;

	TWeakObjectPtr<UDataflow> DataflowGraph = nullptr;

	TWeakPtr<SDataflowGraphEditor> DataflowGraphEditor;

	UPROPERTY()
	TObjectPtr<UEditorInteractiveToolsContext> ActiveToolsContext = nullptr;
	TWeakPtr<UE::Dataflow::FEngineContext> DataflowContext;
	TSharedPtr<FManagedArrayCollection> SelectedClothCollection = nullptr;
	TSharedPtr<FManagedArrayCollection> SelectedInputClothCollection = nullptr;

	// Correspondence between node types and commands to add the node to the graph
	TMap<FName, TSharedPtr<const FUICommandInfo>> NodeTypeToAddNodeCommandMap;

	// Correspondence between node types and commands to launch tools
	TMap<FName, TSharedPtr<const FUICommandInfo>> NodeTypeToToolCommandMap;

	// Timestamps for telemetry
	FDateTime LastModeStartTimestamp;
	FDateTime LastToolStartTimestamp;
};

#undef UE_API
