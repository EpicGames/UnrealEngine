// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorModeToolkit.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorStyle.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "ChaosClothAsset/SClothEditorRestSpaceViewport.h"
#include "ChaosClothAsset/ClothEditorRestSpaceViewportClient.h"
#include "ChaosClothAsset/SClothEditor3DViewport.h"
#include "ChaosClothAsset/ClothEditor3DViewportClient.h"

#include "AssetEditorModeManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "InteractiveToolManager.h"
#include "SEditorViewport.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SBaseCharacterFXEditorViewport.h"
#include "Widgets/Input/SButton.h"
#include "SPrimaryButton.h"
#include "IDetailsView.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FChaosClothAssetEditorModeToolkit"

namespace UE::Chaos::ClothAsset
{

FName FChaosClothAssetEditorModeToolkit::GetToolkitFName() const
{
	return FName("ChaosClothAssetEditorMode");
}

FText FChaosClothAssetEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("ChaosClothAssetEditorModeToolkit", "DisplayName", "ChaosClothAssetEditorMode");
}


void FChaosClothAssetEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);	

	// Set up tool message areas
	ClearNotification();
	ClearWarning();
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolNotificationMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext()->OnToolWarningMessage.AddSP(this, &FBaseCharacterFXEditorModeToolkit::PostWarning);

	// Hook up the tool detail panel
	ToolDetailsContainer->SetContent(DetailsView.ToSharedRef());
	
	// Set up the overlay -- it will poll different ToolsContexts depending on which one launched the currently active tool

	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FBaseCharacterFXEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { 
				    GetCurrentToolsContext()->EndTool(EToolShutdownType::Accept);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetCurrentToolsContext()->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.TextStyle( FAppStyle::Get(), "DialogButtonText" )
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetCurrentToolsContext()->EndTool(EToolShutdownType::Cancel);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() { return GetCurrentToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { 
					GetCurrentToolsContext()->EndTool(EToolShutdownType::Completed);
					return FReply::Handled(); 
					})
				.IsEnabled_Lambda([this]() {
					return GetCurrentToolsContext()->CanCompleteActiveTool();
				})
				.Visibility_Lambda([this]() { return GetCurrentToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];
}

const FSlateBrush* FChaosClothAssetEditorModeToolkit::GetActiveToolIcon(const FString& ActiveToolIdentifier) const
{
	FName ActiveToolIconName = ISlateStyle::Join(FChaosClothAssetEditorCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	return FChaosClothAssetEditorStyle::Get().GetOptionalBrush(ActiveToolIconName);
}

UEditorInteractiveToolsContext* FChaosClothAssetEditorModeToolkit::GetCurrentToolsContext()
{
	if (OwningEditorMode.IsValid(/*bEvenIfPendingKill*/ false))
	{
		UEdMode* const Mode = OwningEditorMode.Get();
		if (UChaosClothAssetEditorMode* const ClothEdMode = Cast<UChaosClothAssetEditorMode>(Mode))
		{
			return ClothEdMode->GetActiveToolsContext();
		}
	}

	// this should not happen, but don't crash if it does
	return GetEditorModeManager().GetInteractiveToolsContext();
}

void FChaosClothAssetEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	ensure(Tool == Manager->GetActiveTool(EToolSide::Left));

	if (!Tool)
	{
		return;
	}

	if (DetailsView)
	{
		DetailsView->SetObjects(Tool->GetToolProperties());

		Tool->OnPropertySetsModified.AddLambda([this, Manager]()
		{
			const UInteractiveTool* const Tool = Manager->GetActiveTool(EToolSide::Left);
			if (Tool != nullptr)
			{
				DetailsView->SetObjects(Tool->GetToolProperties(true));
			}
		});
	}

	Tool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FChaosClothAssetEditorModeToolkit::InvalidateCachedDetailPanelState);

	ActiveToolName = Tool->GetToolInfo().ToolDisplayName;
	FString ActiveToolIdentifier = Manager->GetActiveToolName(EToolSide::Mouse);
	ActiveToolIdentifier.InsertAt(0, ".");
	ActiveToolIcon = GetActiveToolIcon(ActiveToolIdentifier);

	if (TSharedPtr<SBaseCharacterFXEditorViewport> Widget = RestSpaceViewportWidget.Pin())
	{
		Widget->AddOverlayWidget(ViewportOverlayWidget.ToSharedRef());

		// Manually set keyboard focus to this viewport. Otherwise the user has to click on the viewport before Tool keyboard shortcuts will work.
		FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), Widget->AsShared(), EFocusCause::SetDirectly);
	}
}


void FChaosClothAssetEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	FModeToolkit::OnToolEnded(Manager, Tool);

	ActiveToolName = FText::GetEmpty();
	ClearNotification();
	ClearWarning();
	
	if (TSharedPtr<SBaseCharacterFXEditorViewport> Widget = RestSpaceViewportWidget.Pin())
	{
		Widget->RemoveOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	if (UInteractiveTool* const CurTool = Manager->GetActiveTool(EToolSide::Left))
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}
}

void FChaosClothAssetEditorModeToolkit::SetRestSpaceViewportWidget(TWeakPtr<SChaosClothAssetEditorRestSpaceViewport> InRestSpaceViewportWidget)
{
	RestSpaceViewportWidget = InRestSpaceViewportWidget;
}

} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
