// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeToolkit.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UEditorInteractiveToolsContext;
class SChaosClothAssetEditorRestSpaceViewport;
class SChaosClothAssetEditor3DViewport;
class SBaseCharacterFXEditorViewport;

/**
 * The cloth editor mode toolkit is responsible for the panel on the side in the cloth editor
 * that shows mode and tool properties. Tool buttons would go in Init().
 * NOTE: the cloth editor has two separate viewports/worlds/modemanagers/toolscontexts, so we need to track which
 * one is currently active.
 */
namespace UE::Chaos::ClothAsset
{
class FChaosClothAssetEditorModeToolkit : public FBaseCharacterFXEditorModeToolkit
{
public:

	UE_API void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode);

	UE_API virtual const FSlateBrush* GetActiveToolIcon(const FString& Identifier) const override;

	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	// IToolkit
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;

	UE_API void SetRestSpaceViewportWidget(TWeakPtr<SChaosClothAssetEditorRestSpaceViewport>);

private:

	UE_API UEditorInteractiveToolsContext* GetCurrentToolsContext();

	TWeakPtr<SChaosClothAssetEditorRestSpaceViewport> RestSpaceViewportWidget;
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
