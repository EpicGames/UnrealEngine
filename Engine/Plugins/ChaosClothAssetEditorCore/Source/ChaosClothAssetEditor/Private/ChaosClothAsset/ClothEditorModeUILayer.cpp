// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorModeUILayer.h"
#include "ChaosClothAsset/ClothEditorToolkit.h"
#include "ChaosClothAsset/ChaosClothAssetEditorModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Toolkits/IToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorModeUILayer)

const FName UChaosClothAssetEditorUISubsystem::EditorSidePanelAreaName = "ChaosClothAssetEditorSidePanelArea";


void UChaosClothAssetEditorUISubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	using namespace UE::Chaos::ClothAsset;
	FChaosClothAssetEditorModule& ChaosClothAssetEditorModule = FModuleManager::GetModuleChecked<FChaosClothAssetEditorModule>("ChaosClothAssetEditor");
	ChaosClothAssetEditorModule.OnRegisterLayoutExtensions().AddUObject(this, &UChaosClothAssetEditorUISubsystem::RegisterLayoutExtensions);
}

void UChaosClothAssetEditorUISubsystem::Deinitialize()
{
	using namespace UE::Chaos::ClothAsset;
	FChaosClothAssetEditorModule& ChaosClothAssetEditorModule = FModuleManager::GetModuleChecked<FChaosClothAssetEditorModule>("ChaosClothAssetEditor");
	ChaosClothAssetEditorModule.OnRegisterLayoutExtensions().RemoveAll(this);
}

void UChaosClothAssetEditorUISubsystem::RegisterLayoutExtensions(FLayoutExtender& Extender)
{
	FTabManager::FTab NewTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
	Extender.ExtendStack(UChaosClothAssetEditorUISubsystem::EditorSidePanelAreaName, ELayoutExtensionPosition::After, NewTab);
}


