// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeUILayer.h"
#include "ClothEditorModeUILayer.generated.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

/** Interchange layer to manage built in tab locations within the editor's layout. **/
UCLASS(MinimalAPI)
class UChaosClothAssetEditorUISubsystem : public UAssetEditorUISubsystem
{
	GENERATED_BODY()
public:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void RegisterLayoutExtensions(FLayoutExtender& Extender) override;

	// Identifier for the Layout extension in ChaosClothAssetEditorToolkit's StandaloneDefaultLayout
	static UE_API const FName EditorSidePanelAreaName;

};

/** Handles the hosting of additional toolkits, such as the mode toolkit, within the ChaosClothAssetEditor's toolkit. **/
class FChaosClothAssetEditorModeUILayer : public FBaseCharacterFXEditorModeUILayer
{
public:
	FChaosClothAssetEditorModeUILayer(const IToolkitHost* InToolkitHost) : FBaseCharacterFXEditorModeUILayer(InToolkitHost) {}
};

#undef UE_API
