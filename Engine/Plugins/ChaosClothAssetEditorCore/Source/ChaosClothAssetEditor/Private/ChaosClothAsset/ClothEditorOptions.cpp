// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorOptions)

UChaosClothEditorOptions::UChaosClothEditorOptions(class FObjectInitializer const& ObjectInitializer)
{
	bClothAssetsOpenInDataflowEditor = false;
	ConstructionViewportMousePanButton = EConstructionViewportMousePanButton::Right;
}

FName UChaosClothEditorOptions::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR

FText UChaosClothEditorOptions::GetSectionText() const
{
	return NSLOCTEXT("ChaosClothEditorPlugin", "ChaosClothEditorSettingsSection", "Chaos Cloth Editor");
}

#endif
