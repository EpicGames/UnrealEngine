// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "ClothEditorOptions.generated.h"

UENUM(BlueprintType)
enum class EConstructionViewportMousePanButton : uint8
{
	/** Right Mouse Button */
	Right = 0,

	/** Middle Mouse Button */
	Middle = 1,

	/** Either Right or Middle Mouse Buttons */
	RightOrMiddle = 2,
};


/** A settings class used to store and retreive user settings related to the Cloth Editor */
UCLASS(HideCategories = Object, Config = EditorPerProjectUserSettings, Meta = (DisplayName = "Chaos Cloth Editor"), MinimalAPI)
class UChaosClothEditorOptions : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	/** Whether Chaos Cloth Assets open in the Dataflow Editor (vs in the Cloth Editor) */
	UPROPERTY(Config)
	bool bClothAssetsOpenInDataflowEditor;

	/** Which mouse button controls camera panning in the Construction Viewport in 2D view mode */
	UPROPERTY(Config, EditAnywhere, Category = UI)
	EConstructionViewportMousePanButton ConstructionViewportMousePanButton;

	 // UDeveloperSettings overrides
	CHAOSCLOTHASSETEDITOR_API virtual FName GetCategoryName() const override;

#if WITH_EDITOR
	CHAOSCLOTHASSETEDITOR_API virtual FText GetSectionText() const override;
#endif

};
