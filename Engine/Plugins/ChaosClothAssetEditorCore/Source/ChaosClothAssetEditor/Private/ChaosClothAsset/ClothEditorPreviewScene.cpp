// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothEditor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/SkeletalMeshActor.h"
#include "Engine/SkeletalMesh.h"
#include "AssetEditorModeManager.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "ComponentReregisterContext.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "SkinnedAssetCompiler.h"
#include "Misc/TransactionObjectEvent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Transforms/TransformGizmoDataBinder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UObject/PackageReload.h"
#include "EditorReimportHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorPreviewScene)

#define LOCTEXT_NAMESPACE "UChaosClothEditorPreviewScene"

namespace UE::Chaos::ClothAsset::Private
{
	void ValidateClothComponentAttachmentBones(const UChaosClothComponent& ClothComponent)
	{
		const UChaosClothAsset* const ClothAsset = Cast<const UChaosClothAsset>(ClothComponent.GetAsset());
		const FSkeletalMeshRenderData* const ClothRenderData = ClothComponent.GetSkeletalMeshRenderData();
		const USkeletalMeshComponent* const SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ClothComponent.LeaderPoseComponent.Get());
		if (ClothAsset && ClothRenderData && SkeletalMeshComponent)
		{
			bool bAnyMissingBones = false;

			const TArray<int32>& LeaderBoneMap = ClothComponent.GetLeaderBoneMap();
			const FReferenceSkeleton& RefSkeleton = ClothAsset->GetRefSkeleton();
			auto CheckRequiredBones = [&LeaderBoneMap](const TArray<FBoneIndexType>& RequiredFollowerBones,
				const TArray<FBoneIndexType>& AvailableLeaderBones, TArray<FBoneIndexType>& OutMissingBones)
			{
				OutMissingBones.Reset();
				if (RequiredFollowerBones.IsEmpty())
				{
					return;
				}
				TSet<FBoneIndexType> AvailableLeaderBonesSet(AvailableLeaderBones);
				for (const FBoneIndexType RequiredBone : RequiredFollowerBones)
				{
					if (!LeaderBoneMap.IsValidIndex(RequiredBone) || LeaderBoneMap[RequiredBone] == INDEX_NONE ||
						!AvailableLeaderBonesSet.Contains(LeaderBoneMap[RequiredBone]))
					{
						OutMissingBones.Add(RequiredBone);
					}
				}
			};

			TArray<FBoneIndexType> PhysAssetBones;
			if (const UPhysicsAsset* ClothPhysAsset = ClothComponent.GetPhysicsAsset())
			{
				USkinnedMeshComponent::GetPhysicsRequiredBones(ClothAsset, ClothPhysAsset, PhysAssetBones);
			}

			TArray<FBoneIndexType> MissingBones;
			for (int32 LODIndex = SkeletalMeshComponent->ComputeMinLOD(); LODIndex < SkeletalMeshComponent->GetNumLODs(); ++LODIndex)
			{
				// Check all Leader SKM LODs since the Cloth should try to follow the SKM LOD.
				// These should be the bones that will actually be calculated by the SKM (see USkeletalMeshComponent::RecalcRequiredBones)
				TArray<FBoneIndexType> SKMRequiredBones;
				TArray<FBoneIndexType> SKMFillComponentSpaceTransformsRequiredBones;
				SkeletalMeshComponent->ComputeRequiredBones(SKMRequiredBones, SKMFillComponentSpaceTransformsRequiredBones, LODIndex, /*bIgnorePhysicsAsset*/ false);

				CheckRequiredBones(PhysAssetBones, SKMFillComponentSpaceTransformsRequiredBones, MissingBones);
				if (MissingBones.Num())
				{
					FText MissingBonesMsg = FText::FormatOrdered(LOCTEXT("MissingPhysicsBones", "SkeletalMesh \"{0}\" (LOD {1}) will not update the following bones required by \"{2}\"'s PhysicsAsset ({3}): "),
						FText::FromString(SkeletalMeshComponent->GetSkinnedAsset()->GetName()),
						FText::AsNumber(LODIndex), FText::FromString(ClothAsset->GetName()), FText::FromString(ClothComponent.GetPhysicsAsset()->GetName()));
					for (const FBoneIndexType MissingBone : MissingBones)
					{
						MissingBonesMsg = FText::Format(LOCTEXT("MissingPhysicsBoneList", "{0} {1}"), MissingBonesMsg, 
							FText::FromName(RefSkeleton.GetBoneName(MissingBone)));
					}
					UE_LOGF(LogChaosClothAssetEditor, Warning, "%ls", *MissingBonesMsg.ToString());
					bAnyMissingBones = true;
				}
			}

			if (bAnyMissingBones)
			{
				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("MissingBonesNotification", "Cloth asset {0} is not compatible with the preview skeletal mesh {1} (missing bones). See log for more details."),
					FText::FromString(ClothAsset->GetName()), FText::FromString(SkeletalMeshComponent->GetSkinnedAsset()->GetName())));
				NotificationInfo.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
		}
	}
}

void UChaosClothPreviewSceneDescription::SetPreviewScene(UE::Chaos::ClothAsset::FChaosClothPreviewScene* InPreviewScene)
{
	PreviewScene = InPreviewScene;
}

void UChaosClothPreviewSceneDescription::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PreviewScene)
	{
		PreviewScene->SceneDescriptionPropertyChanged(PropertyChangedEvent.GetMemberPropertyName());
	}

	ClothPreviewSceneDescriptionChanged.Broadcast();
}

void UChaosClothPreviewSceneDescription::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	// On Undo/Redo, PostEditChangeProperty just gets an empty FPropertyChangedEvent. However this function gets enough info to figure out which property changed
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo && TransactionEvent.HasPropertyChanges())
	{
		const TArray<FName>& PropertyNames = TransactionEvent.GetChangedProperties();
		for (const FName& PropertyName : PropertyNames)
		{
			PreviewScene->SceneDescriptionPropertyChanged(PropertyName);
		}
	}
}

namespace UE::Chaos::ClothAsset
{

FChaosClothPreviewScene::FChaosClothPreviewScene(FPreviewScene::ConstructionValues ConstructionValues) :
	FAdvancedPreviewScene(ConstructionValues)
{
	PreviewSceneDescription = NewObject<UChaosClothPreviewSceneDescription>();
	PreviewSceneDescription->SetPreviewScene(this);

	SceneActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass());

	SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(SceneActor);
	SkeletalMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FChaosClothPreviewScene::IsComponentSelected);
	SkeletalMeshComponent->SetDisablePostProcessBlueprint(false);
	SkeletalMeshComponent->RegisterComponentWithWorld(GetWorld());

	ClothComponent = NewObject<UChaosClothComponent>(SceneActor);
	ClothComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FChaosClothPreviewScene::IsComponentSelected);
	ClothComponent->RegisterComponentWithWorld(GetWorld());
	
	OnPackageReloadedDelegateHandle = FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FChaosClothPreviewScene::HandlePackageReloaded);
	OnPostReimportDelegateHandle = FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FChaosClothPreviewScene::HandleReimportManagerPostReimport);
}

FChaosClothPreviewScene::~FChaosClothPreviewScene()
{
	if (SkeletalMeshComponent)
	{
		SkeletalMeshComponent->TransformUpdated.RemoveAll(this);
		SkeletalMeshComponent->SelectionOverrideDelegate.Unbind();
		SkeletalMeshComponent->UnregisterComponent();
	}

	if (ClothComponent)
	{
		ClothComponent->SelectionOverrideDelegate.Unbind();
		ClothComponent->UnregisterComponent();
	}

	FCoreUObjectDelegates::OnPackageReloaded.Remove(OnPackageReloadedDelegateHandle);
	FReimportManager::Instance()->OnPostReimport().Remove(OnPostReimportDelegateHandle);
}

void FChaosClothPreviewScene::AddReferencedObjects(FReferenceCollector& Collector)
{
	FAdvancedPreviewScene::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(PreviewSceneDescription);
	Collector.AddReferencedObject(ClothComponent);
	Collector.AddReferencedObject(SkeletalMeshComponent);
	Collector.AddReferencedObject(SceneActor);
}

void FChaosClothPreviewScene::Tick(float DeltaT)
{
	FAdvancedPreviewScene::Tick(DeltaT);

	if (SavedAnimState)
	{
		RestoreSavedAnimationState();
	}
}

void FChaosClothPreviewScene::UpdateSkeletalMeshAnimation()
{
	check(SkeletalMeshComponent);

	const bool bWasPlaying = SkeletalMeshComponent->IsPlaying();
	SkeletalMeshComponent->Stop();

	if (PreviewSceneDescription->AnimationAsset)
	{
		TObjectPtr<UAnimSingleNodeInstance> PreviewAnimInstance = NewObject<UAnimSingleNodeInstance>(SkeletalMeshComponent);
		PreviewAnimInstance->SetAnimationAsset(PreviewSceneDescription->AnimationAsset);

		SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
		SkeletalMeshComponent->InitAnim(true);
		SkeletalMeshComponent->AnimationData.PopulateFrom(PreviewAnimInstance);
		SkeletalMeshComponent->AnimScriptInstance = PreviewAnimInstance;
		SkeletalMeshComponent->AnimScriptInstance->InitializeAnimation();
		SkeletalMeshComponent->ValidateAnimation();

		if (!bWasPlaying)
		{
			SkeletalMeshComponent->Stop();
		}
	}
	else
	{
		SkeletalMeshComponent->AnimationData = FSingleAnimationPlayData();
		SkeletalMeshComponent->AnimScriptInstance = nullptr;
	}
}

void FChaosClothPreviewScene::UpdateClothComponentAttachment()
{
	check(SkeletalMeshComponent);
	check(ClothComponent);

	if (SkeletalMeshComponent->GetSkeletalMeshAsset() && !ClothComponent->IsAttachedTo(SkeletalMeshComponent))
	{
		ClothComponent->AttachToComponent(SkeletalMeshComponent, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
	}
	else if (!SkeletalMeshComponent->GetSkeletalMeshAsset() && ClothComponent->IsAttachedTo(SkeletalMeshComponent))
	{
		ClothComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

		// Hard reset cloth simulation if we are losing the attachment
		{
			const FComponentReregisterContext Context(ClothComponent);
		}
	}
}

void FChaosClothPreviewScene::SceneDescriptionPropertyChanged(const FName& PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, SkeletalMeshAsset))
	{
		check(SkeletalMeshComponent);
		
		SaveAnimationState();

		SkeletalMeshComponent->SetSkeletalMeshAsset(PreviewSceneDescription->SkeletalMeshAsset);

		UpdateSkeletalMeshAnimation();
		UpdateClothComponentAttachment();

		UE::Chaos::ClothAsset::Private::ValidateClothComponentAttachmentBones(*ClothComponent);

		if (UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(ClothComponent->GetAsset()))
		{
			ClothAsset->SetPreviewSceneSkeletalMesh(PreviewSceneDescription->SkeletalMeshAsset);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, Translation) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, Rotation) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, Scale))
	{
		if (DataBinder)
		{
			DataBinder->UpdateAfterDataEdit();
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, AnimationAsset))
	{
		UpdateSkeletalMeshAnimation();

		if (UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(ClothComponent->GetAsset()))
		{
			ClothAsset->SetPreviewSceneAnimation(PreviewSceneDescription->AnimationAsset);
		}
	}
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, bPostProcessBlueprint))
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->SetDisablePostProcessBlueprint(!PreviewSceneDescription->bPostProcessBlueprint);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, SolverGeometryScale))
	{
		if (ClothComponent)
		{
			ClothComponent->SetClothGeometryScale(PreviewSceneDescription->SolverGeometryScale);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, SolverGeometryScale))
	{
		if (ClothComponent)
		{
			ClothComponent->SetClothGeometryScale(PreviewSceneDescription->SolverGeometryScale);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, TeleportDistanceThreshold))
	{
		if (ClothComponent)
		{
			ClothComponent->SetTeleportDistanceThreshold(PreviewSceneDescription->TeleportDistanceThreshold);
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UChaosClothPreviewSceneDescription, TeleportRotationThreshold))
	{
		if (ClothComponent)
		{
			ClothComponent->SetTeleportRotationThreshold(PreviewSceneDescription->TeleportRotationThreshold);
		}
	}
}

UChaosClothComponent* FChaosClothPreviewScene::GetClothComponent()
{
	return ClothComponent;
}

const UChaosClothComponent* FChaosClothPreviewScene::GetClothComponent() const
{
	return ClothComponent;
}

const USkeletalMeshComponent* FChaosClothPreviewScene::GetSkeletalMeshComponent() const
{
	return SkeletalMeshComponent;
}

void FChaosClothPreviewScene::SetModeManager(const TSharedPtr<FAssetEditorModeManager>& InClothPreviewEditorModeManager)
{
	ClothPreviewEditorModeManager = InClothPreviewEditorModeManager;
}

bool FChaosClothPreviewScene::IsComponentSelected(const UPrimitiveComponent* InComponent)
{
	if (const UTypedElementSelectionSet* const TypedElementSelectionSet = ClothPreviewEditorModeManager->GetEditorSelectionSet())
	{
		if (const FTypedElementHandle ComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent))
		{
			const bool bElementSelected = TypedElementSelectionSet->IsElementSelected(ComponentElement, FTypedElementIsSelectedOptions());
			return bElementSelected;
		}
	}

	return false;
}

void FChaosClothPreviewScene::SetClothAsset(UChaosClothAsset* Asset)
{
	check(Asset);
	check(SceneActor);
	check(ClothComponent);

	if (Cast<UChaosClothAsset>(ClothComponent->GetAsset()) != Asset)
	{
		ClothComponent->SetAsset(Asset);
	}

	UpdateClothComponentAttachment();

	// Wait for asset to load and update the component bounds
	ClothComponent->InvalidateCachedBounds();
	FSkinnedAssetCompilingManager::Get().FinishCompilation(TArrayView<USkinnedAsset* const>{Asset});
	ClothComponent->UpdateBounds();

	if (USkeletalMesh* const SkeletalMesh = Asset->GetPreviewSceneSkeletalMesh())
	{
		if (SkeletalMesh != PreviewSceneDescription->SkeletalMeshAsset)
		{
			PreviewSceneDescription->SkeletalMeshAsset = SkeletalMesh;

			SkeletalMeshComponent->SetSkeletalMeshAsset(PreviewSceneDescription->SkeletalMeshAsset);

			UpdateSkeletalMeshAnimation();
			UpdateClothComponentAttachment();
		}
	}
	
	if (UAnimationAsset* const Animation = Asset->GetPreviewSceneAnimation())
	{
		if (PreviewSceneDescription->AnimationAsset != Animation)
		{
			PreviewSceneDescription->AnimationAsset = Animation;

			UpdateSkeletalMeshAnimation();
		}
	}
}

UAnimSingleNodeInstance* FChaosClothPreviewScene::GetPreviewAnimInstance()
{
	check(SkeletalMeshComponent);

	if (SkeletalMeshComponent->AnimScriptInstance)
	{
		return CastChecked<UAnimSingleNodeInstance>(SkeletalMeshComponent->AnimScriptInstance);
	}
	return nullptr;
}

const UAnimSingleNodeInstance* const FChaosClothPreviewScene::GetPreviewAnimInstance() const
{
	check(SkeletalMeshComponent);

	if (SkeletalMeshComponent->AnimScriptInstance)
	{
		return CastChecked<UAnimSingleNodeInstance>(SkeletalMeshComponent->AnimScriptInstance);
	}
	return nullptr;
}

void FChaosClothPreviewScene::SetGizmoDataBinder(TSharedPtr<FTransformGizmoDataBinder> InDataBinder)
{
	DataBinder = InDataBinder;
}


void FChaosClothPreviewScene::SaveAnimationState()
{
	if (const UAnimSingleNodeInstance* const AnimInstance = GetPreviewAnimInstance())
	{
		SavedAnimState = FAnimState();
		SavedAnimState->Time = AnimInstance->GetCurrentTime();
		SavedAnimState->bIsReverse = AnimInstance->IsReverse();
		SavedAnimState->bIsLooping = AnimInstance->IsLooping();
		SavedAnimState->bIsPlaying = AnimInstance->IsPlaying();
	}
}

void FChaosClothPreviewScene::RestoreSavedAnimationState()
{
	if (SavedAnimState)
	{
		if (UAnimSingleNodeInstance* const AnimInstance = GetPreviewAnimInstance())
		{
			AnimInstance->SetPosition(SavedAnimState->Time);
			AnimInstance->SetReverse(SavedAnimState->bIsReverse);
			AnimInstance->SetLooping(SavedAnimState->bIsLooping);
			AnimInstance->SetPlaying(SavedAnimState->bIsPlaying);
		}
		SavedAnimState.Reset();
	}
}

void FChaosClothPreviewScene::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)
	{
		for (const TPair<UObject*, UObject*>& RepointPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (RepointPair.Key == PreviewSceneDescription->SkeletalMeshAsset.Get())
			{
				// If we are going to be reloading the SkeletalMesh, first save the animation state since the AnimInstance will be reinitialized when the component is reregistered.
				// Note we restore from the saved state in the Tick function above because AnimInstance reinitialization happens /after/ all reload delegates are called.
				SaveAnimationState();
			}
		}
	}
}

void FChaosClothPreviewScene::HandleReimportManagerPostReimport(UObject* ReimportedObject, bool bWasSuccessful)
{
	if (ReimportedObject == PreviewSceneDescription->SkeletalMeshAsset && bWasSuccessful)
	{
		// If we have reimported the SkeletalMesh, save the animation state since the AnimInstance will be reinitialized when the component is reregistered.
		SaveAnimationState();
	}
}

} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE

