// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "Math/Transform.h"
#include "ClothEditorPreviewScene.generated.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UChaosClothAsset;
class USkeletalMesh;
class UChaosClothComponent;
class FAssetEditorModeManager;
class UAnimationAsset;
class UAnimSingleNodeInstance;
class FTransformGizmoDataBinder;

namespace UE::Chaos::ClothAsset
{
class FChaosClothPreviewScene;
}

DECLARE_EVENT(UChaosClothPreviewSceneDescription, FClothPreviewSceneDescriptionChanged)

///
/// The UChaosClothPreviewSceneDescription is a description of the Preview scene contents, intended to be editable in an FAdvancedPreviewSettingsWidget
/// 
UCLASS(MinimalAPI)
class UChaosClothPreviewSceneDescription : public UObject
{
public:
	GENERATED_BODY()

	FClothPreviewSceneDescriptionChanged ClothPreviewSceneDescriptionChanged;

	UChaosClothPreviewSceneDescription()
	{
		SetFlags(RF_Transactional);
	}

	UE_API void SetPreviewScene(UE::Chaos::ClothAsset::FChaosClothPreviewScene* PreviewScene);

	/* Whether the preview viewport should pause animation and simulation while Play In Editor (PIE) or Simulate In Editor is active */
	UPROPERTY(EditAnywhere, Transient, Category = "Viewport")
	bool bPauseWhilePlayingInEditor = true;

	// Skeletal Mesh source asset
	UPROPERTY(EditAnywhere, Transient, Category="SkeletalMesh")
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset;

	UPROPERTY(EditAnywhere, Transient, Category = "SkeletalMesh")
	TObjectPtr<UAnimationAsset> AnimationAsset;

	UPROPERTY(EditAnywhere, Transient, Category = "SkeletalMesh")
	bool bPostProcessBlueprint;

	UPROPERTY(EditAnywhere, Transient, Category = "Transform", Meta = (EditCondition = "bValidSelectionForTransform", HideEditConditionToggle))
	FVector3d Translation = FVector3d::ZeroVector;

	UPROPERTY(EditAnywhere, Transient, Category = "Transform", Meta = (EditCondition = "bValidSelectionForTransform", HideEditConditionToggle))
	FVector3d Rotation = FVector3d::ZeroVector;

	UPROPERTY(EditAnywhere, Transient, Category = "Transform", Meta = (AllowPreserveRatio, EditCondition = "bValidSelectionForTransform", HideEditConditionToggle))
	FVector3d Scale = FVector3d::OneVector;

	UPROPERTY(EditAnywhere, Transient, Category = "ClothComponent", Meta = (UIMin = 0.0, UIMax = 10.0, ClampMin = 0.0, ClampMax = 10000.0))
	float SolverGeometryScale = 1.f;

	/**
	* Conduct teleportation if the character's movement is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, Transient, Category = ClothComponent)
	float TeleportDistanceThreshold = 0.f;

	/**
	* Rotation threshold in degrees, ranging from 0 to 180.
	* Conduct teleportation if the character's rotation is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, Transient, Category = ClothComponent)
	float TeleportRotationThreshold = 0.f;

	UPROPERTY(Transient)
	bool bValidSelectionForTransform = false;

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

private:

	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;

	UE::Chaos::ClothAsset::FChaosClothPreviewScene* PreviewScene;
};


namespace UE::Chaos::ClothAsset
{
///
/// FChaosClothPreviewScene is the actual Preview scene, with contents specified by the SceneDescription
/// 
class FChaosClothPreviewScene : public FAdvancedPreviewScene
{
public:

	UE_API FChaosClothPreviewScene(FPreviewScene::ConstructionValues ConstructionValues);
	UE_API virtual ~FChaosClothPreviewScene();

	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	const UChaosClothPreviewSceneDescription* GetPreviewSceneDescription() const { return PreviewSceneDescription; }
	UChaosClothPreviewSceneDescription* GetPreviewSceneDescription() { return PreviewSceneDescription; }

	UE_API void SetClothAsset(UChaosClothAsset* Asset);

	// Update Scene in response to the SceneDescription changing
	UE_API void SceneDescriptionPropertyChanged(const FName& PropertyName);

	UE_API UAnimSingleNodeInstance* GetPreviewAnimInstance();
	UE_API const UAnimSingleNodeInstance* const GetPreviewAnimInstance() const;

	UE_API UChaosClothComponent* GetClothComponent();
	UE_API const UChaosClothComponent* GetClothComponent() const;
	
	UE_API const USkeletalMeshComponent* GetSkeletalMeshComponent() const;

	/** Set the scene's ModeManager, which is mainly used to track selected components */
	UE_API void SetModeManager(const TSharedPtr<FAssetEditorModeManager>& InClothPreviewEditorModeManager);

	UE_API void SetGizmoDataBinder(TSharedPtr<FTransformGizmoDataBinder> InDataBinder);

private:

	UE_API virtual void Tick(float DeltaT) override;

	// Create the PreviewAnimationInstance if the AnimationAsset and SkeletalMesh both exist, and set the animation to run on the SkeletalMeshComponent
	UE_API void UpdateSkeletalMeshAnimation();

	// Attach the cloth component to the skeletal mesh component, if it exists
	UE_API void UpdateClothComponentAttachment();

	UE_API bool IsComponentSelected(const UPrimitiveComponent* InComponent);

	UE_API void SaveAnimationState();
	UE_API void RestoreSavedAnimationState();
	UE_API void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);
	UE_API void HandleReimportManagerPostReimport(UObject* ReimportedObject, bool bWasSuccessful);

	TObjectPtr<UChaosClothPreviewSceneDescription> PreviewSceneDescription;

	TSharedPtr<FAssetEditorModeManager> ClothPreviewEditorModeManager;

	TObjectPtr<AActor> SceneActor;

	TObjectPtr<UChaosClothComponent> ClothComponent;

	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	TSharedPtr<FTransformGizmoDataBinder> DataBinder = nullptr;

	struct FAnimState
	{
		float Time;
		bool bIsReverse;
		bool bIsLooping;
		bool bIsPlaying;
	};
	TOptional<FAnimState> SavedAnimState;

	FDelegateHandle OnPackageReloadedDelegateHandle;
	FDelegateHandle OnPostReimportDelegateHandle;
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
