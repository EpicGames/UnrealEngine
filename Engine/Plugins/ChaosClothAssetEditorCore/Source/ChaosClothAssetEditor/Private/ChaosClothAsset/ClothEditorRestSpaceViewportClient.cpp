// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorRestSpaceViewportClient.h"

#include "ChaosClothAsset/ClothEditorOptions.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseWheelBehavior.h"
#include "EngineUtils.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "Behaviors/2DViewportBehaviorTargets.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "CameraController.h"
#include "Components/DynamicMeshComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "SceneView.h"
#include "PreviewScene.h"
#include "Components/PointLightComponent.h"

namespace UE::Chaos::ClothAsset
{

FChaosClothEditorRestSpaceViewportClient::FChaosClothEditorRestSpaceViewportClient(FEditorModeTools* InModeTools,
	FPreviewScene* InPreviewScene,
	const TWeakPtr<SEditorViewport>& InEditorViewportWidget) :
	FEditorViewportClient(InModeTools, InPreviewScene, InEditorViewportWidget)
{
	OverrideNearClipPlane(UE_KINDA_SMALL_NUMBER);
	OverrideFarClipPlane(0);

	BehaviorSet = NewObject<UInputBehaviorSet>();

	// We'll have the priority of our viewport manipulation behaviors be lower (i.e. higher
	// numerically) than both the gizmo default and the tool default.
	constexpr int ViewportBehaviorPriority = 150;

	ScrollBehaviorTarget = MakeUnique<FEditor2DScrollBehaviorTarget>(this);

	bool bUseRightMouseButton = true;
	bool bUseMiddleMouseButton = true;
	if (const UChaosClothEditorOptions* const Options = GetDefault<UChaosClothEditorOptions>())
	{
		bUseRightMouseButton = (Options->ConstructionViewportMousePanButton == EConstructionViewportMousePanButton::Right || Options->ConstructionViewportMousePanButton == EConstructionViewportMousePanButton::RightOrMiddle);
		bUseMiddleMouseButton = (Options->ConstructionViewportMousePanButton == EConstructionViewportMousePanButton::Middle || Options->ConstructionViewportMousePanButton == EConstructionViewportMousePanButton::RightOrMiddle);
	}

	if (bUseRightMouseButton)
	{
		UClickDragInputBehavior* const RightMouseClickDragInputBehavior = NewObject<UClickDragInputBehavior>();
		RightMouseClickDragInputBehavior->Initialize(ScrollBehaviorTarget.Get());
		RightMouseClickDragInputBehavior->SetDefaultPriority(ViewportBehaviorPriority);
		RightMouseClickDragInputBehavior->SetUseRightMouseButton();
		BehaviorsFor2DMode.Add(RightMouseClickDragInputBehavior);
	}

	if (bUseMiddleMouseButton)
	{
		UClickDragInputBehavior* const MiddleMouseClickDragInputBehavior = NewObject<UClickDragInputBehavior>();
		MiddleMouseClickDragInputBehavior->Initialize(ScrollBehaviorTarget.Get());
		MiddleMouseClickDragInputBehavior->SetDefaultPriority(ViewportBehaviorPriority);
		MiddleMouseClickDragInputBehavior->SetUseMiddleMouseButton();
		BehaviorsFor2DMode.Add(MiddleMouseClickDragInputBehavior);
	}

	ZoomBehaviorTarget = MakeUnique<FEditor2DMouseWheelZoomBehaviorTarget>(this);
	constexpr float CameraFarPlaneWorldZ = -10.0f;
	constexpr float CameraNearPlaneProportionZ = 0.8f;
	ZoomBehaviorTarget->SetCameraFarPlaneWorldZ(CameraFarPlaneWorldZ);
	ZoomBehaviorTarget->SetCameraNearPlaneProportionZ(CameraNearPlaneProportionZ);
	ZoomBehaviorTarget->SetZoomLimits(0.001, 100000);
	UMouseWheelInputBehavior* const ZoomBehavior = NewObject<UMouseWheelInputBehavior>();
	ZoomBehavior->Initialize(ZoomBehaviorTarget.Get());
	ZoomBehavior->SetDefaultPriority(ViewportBehaviorPriority);
	BehaviorsFor2DMode.Add(ZoomBehavior);


	const TObjectPtr<ULocalClickDragInputBehavior> ClickDrag3DBehavior = NewObject<ULocalClickDragInputBehavior>();
	ClickDrag3DBehavior->Initialize();
	ClickDrag3DBehavior->SetDefaultPriority(ViewportBehaviorPriority);

	ClickDrag3DBehavior->ModifierCheckFunc = [this](const FInputDeviceState& InputState) {
		return !FInputDeviceState::IsAltKeyDown(InputState);
	};

	ClickDrag3DBehavior->CanBeginClickDragFunc = [](const FInputDeviceRay& InputDeviceRay)
	{
		return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker.
	};

	BehaviorsFor3DMode.Add(ClickDrag3DBehavior);

	UpdateBehaviorsForCurrentViewMode();

	EngineShowFlags.SetSelectionOutline(false);
	ModeTools->GetInteractiveToolsContext()->InputRouter->RegisterSource(this);

	CameraPointLight = NewObject<UPointLightComponent>();
	CameraPointLight->bUseInverseSquaredFalloff = false;
	CameraPointLight->LightFalloffExponent = 2.0f;
	CameraPointLight->SetIntensity(3.0f);
	CameraPointLight->SetCastShadows(false);
	PreviewScene->AddComponent(CameraPointLight, FTransform());
}

void FChaosClothEditorRestSpaceViewportClient::SetConstructionViewMode(EClothPatternVertexType InViewMode)
{
	const bool bSwitching2D3D = (ConstructionViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D) != (InViewMode == UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D);
	if (bSwitching2D3D)
	{
		Swap(SavedInactiveViewTransform, ViewTransformPerspective);
	}

	ConstructionViewMode = InViewMode;

	if (ConstructionViewMode == EClothPatternVertexType::Sim2D)
	{
		const double AbsZ = FMath::Abs(ViewTransformPerspective.GetLocation().Z);
		constexpr double CameraFarPlaneWorldZ = -10.0;
		constexpr double CameraNearPlaneProportionZ = 0.8;
		OverrideFarClipPlane(static_cast<float>(AbsZ - CameraFarPlaneWorldZ));
		OverrideNearClipPlane(static_cast<float>(AbsZ * (1.0 - CameraNearPlaneProportionZ)));
	}
	else
	{
		OverrideFarClipPlane(0);
		OverrideNearClipPlane(UE_KINDA_SMALL_NUMBER);
	}

	UpdateBehaviorsForCurrentViewMode();

	ModeTools->GetInteractiveToolsContext()->InputRouter->DeregisterSource(this);
	ModeTools->GetInteractiveToolsContext()->InputRouter->RegisterSource(this);
}

void FChaosClothEditorRestSpaceViewportClient::UpdateBehaviorsForCurrentViewMode()
{
	BehaviorSet->RemoveAll();

	if (ConstructionViewMode == EClothPatternVertexType::Sim2D)
	{
		for (UInputBehavior* const Behavior : BehaviorsFor2DMode)
		{
			BehaviorSet->Add(Behavior);
		}
	}
	else
	{
		for (UInputBehavior* const Behavior : BehaviorsFor3DMode)
		{
			BehaviorSet->Add(Behavior);
		}
	}
}

EClothPatternVertexType FChaosClothEditorRestSpaceViewportClient::GetConstructionViewMode() const
{
	return ConstructionViewMode;
}

const UInputBehaviorSet* FChaosClothEditorRestSpaceViewportClient::GetInputBehaviors() const
{
	return BehaviorSet;
}

// Collects UObjects that we don't want the garbage collecter to clean up
void FChaosClothEditorRestSpaceViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEditorViewportClient::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(BehaviorSet);
	Collector.AddReferencedObjects(BehaviorsFor2DMode);
	Collector.AddReferencedObjects(BehaviorsFor3DMode);
}

bool FChaosClothEditorRestSpaceViewportClient::ShouldOrbitCamera() const
{
	if (ConstructionViewMode == EClothPatternVertexType::Sim2D)
	{
		return false;
	}
	else
	{
		return FEditorViewportClient::ShouldOrbitCamera();
	}
}

void FChaosClothEditorRestSpaceViewportClient::Tick(float DeltaSeconds)
{
	FViewportCameraTransform ViewTransform = GetViewTransform();
	CameraPointLight->SetRelativeLocation(ViewTransform.GetLocation());

	FEditorViewportClient::Tick(DeltaSeconds);
}


bool FChaosClothEditorRestSpaceViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	// See if any tool commands want to handle the key event
	const TSharedPtr<FUICommandList> PinnedToolCommandList = ToolCommandList.Pin();
	if (EventArgs.Event != IE_Released && PinnedToolCommandList.IsValid())
	{
		const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		if (PinnedToolCommandList->ProcessCommandBindings(EventArgs.Key, KeyState, (EventArgs.Event == IE_Repeat)))
		{
			return true;
		}
	}

	if (ConstructionViewMode != EClothPatternVertexType::Sim2D)
	{
		return FEditorViewportClient::InputKey(EventArgs);
	}

	// We'll support disabling input like our base class, even if it does not end up being used.
	if (bDisableInput)
	{
		return true;
	}

	// Our viewport manipulation is placed in the input router that ModeTools manages
	return ModeTools->InputKey(this, EventArgs.Viewport, EventArgs.Key, EventArgs.Event);
}

void FChaosClothEditorRestSpaceViewportClient::SetEditorViewportWidget(TWeakPtr<SEditorViewport> InEditorViewportWidget)
{
	EditorViewportWidget = InEditorViewportWidget;
}

void FChaosClothEditorRestSpaceViewportClient::SetToolCommandList(TWeakPtr<FUICommandList> InToolCommandList)
{
	ToolCommandList = InToolCommandList;
}

float FChaosClothEditorRestSpaceViewportClient::GetCameraPointLightIntensity() const
{
	return CameraPointLight->Intensity;
}

void FChaosClothEditorRestSpaceViewportClient::SetCameraPointLightIntensity(float Intensity)
{
	CameraPointLight->SetIntensity(Intensity);
}


} // namespace UE::Chaos::ClothAsset
