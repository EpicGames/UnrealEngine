// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorViewportClient.h"
#include "Behaviors/2DViewportBehaviorTargets.h" // FEditor2DScrollBehaviorTarget, FEditor2DMouseWheelZoomBehaviorTarget
#include "InputBehaviorSet.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UInputBehaviorSet;
class UPointLightComponent;

namespace UE::Chaos::ClothAsset
{

class FChaosClothEditorRestSpaceViewportClient : public FEditorViewportClient, public IInputBehaviorSource
{
public:

	UE_API FChaosClothEditorRestSpaceViewportClient(FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene = nullptr, const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	virtual ~FChaosClothEditorRestSpaceViewportClient() = default;

	// IInputBehaviorSource
	UE_API virtual const UInputBehaviorSet* GetInputBehaviors() const override;

	// FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	UE_API virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;

	UE_API virtual bool ShouldOrbitCamera() const override;

	UE_API void SetConstructionViewMode(EClothPatternVertexType InViewMode);
	UE_API EClothPatternVertexType GetConstructionViewMode() const;

	UE_API void SetEditorViewportWidget(TWeakPtr<SEditorViewport> InEditorViewportWidget);
	UE_API void SetToolCommandList(TWeakPtr<FUICommandList> ToolCommandList);

	UE_API float GetCameraPointLightIntensity() const;
	UE_API void SetCameraPointLightIntensity(float Intensity);

private:

	UE_API virtual void Tick(float DeltaSeconds) override;

	UE_API void UpdateBehaviorsForCurrentViewMode();

	TObjectPtr<UPointLightComponent> CameraPointLight;

	EClothPatternVertexType ConstructionViewMode = EClothPatternVertexType::Sim2D;

	TObjectPtr<UInputBehaviorSet> BehaviorSet;

	TArray<TObjectPtr<UInputBehavior>> BehaviorsFor2DMode;
	TArray<TObjectPtr<UInputBehavior>> BehaviorsFor3DMode;

	TUniquePtr<FEditor2DScrollBehaviorTarget> ScrollBehaviorTarget;
	TUniquePtr<FEditor2DMouseWheelZoomBehaviorTarget> ZoomBehaviorTarget;

	TWeakPtr<FUICommandList> ToolCommandList;

	// Saved view transform for the currently inactive view mode (i.e. store the 3D camera here while in 2D mode and vice-versa)
	FViewportCameraTransform SavedInactiveViewTransform;
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
