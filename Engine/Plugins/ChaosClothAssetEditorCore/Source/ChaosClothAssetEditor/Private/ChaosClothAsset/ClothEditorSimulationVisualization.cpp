// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothEditorSimulationVisualization.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothEditor3DViewportClient.h"
#include "ChaosClothAsset/ClothSimulationProxy.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosCloth/ChaosClothVisualization.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Dataflow/DataflowSimulationViewportClient.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Types/SlateEnums.h"
#include "EditorViewportClient.h"


#define LOCTEXT_NAMESPACE "ClothEditorSimulationVisualization"

namespace UE::Chaos::ClothAsset
{
namespace Private
{
FText ConcatenateLine(const FText& InText, const FText& InNewLine)
{
	if (InText.IsEmpty())
	{
		return InNewLine;
	}
	return FText::Format(LOCTEXT("ViewportTextNewlineFormatter", "{0}\n{1}"), InText, InNewLine);
}

static FText GetSimulationStatisticsString(const FClothSimulationProxy& SimProxy)
{
	FText TextValue;
	// Cloth stats
	if (const int32 NumActiveCloths = SimProxy.GetNumCloths())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumActiveCloths", "Active Cloths: {0}"), NumActiveCloths));
	}
	if (const int32 NumKinematicParticles = SimProxy.GetNumKinematicParticles())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumKinematicParticles", "Kinematic Particles: {0}"), NumKinematicParticles));
	}
	if (const int32 NumDynamicParticles = SimProxy.GetNumDynamicParticles())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumDynamicParticles", "Dynamic Particles: {0}"), NumDynamicParticles));
	}
	if (const int32 NumIterations = SimProxy.GetNumIterations())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumIterations", "Iterations: {0}"), NumIterations));
	}
	if (const int32 NumSubSteps = SimProxy.GetNumSubsteps())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumSubsteps", "Substeps: {0}"), NumSubSteps));
	}
	if (const int32 NumLinearSolveIterations = SimProxy.GetNumLinearSolveIterations())
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumCGIterations", "CGIterations: {0}"), NumLinearSolveIterations));
	}
	if (const float LinearSolveError = SimProxy.GetLinearSolveError())
	{
		FNumberFormattingOptions NumberFormatOptions;
		NumberFormatOptions.AlwaysSign = false;
		NumberFormatOptions.UseGrouping = false;
		NumberFormatOptions.RoundingMode = ERoundingMode::HalfFromZero;
		NumberFormatOptions.MinimumIntegralDigits = 1;
		NumberFormatOptions.MaximumIntegralDigits = 6;
		NumberFormatOptions.MinimumFractionalDigits = 2;
		NumberFormatOptions.MaximumFractionalDigits = 6;
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("CGError", "CGError: {0}"), FText::AsNumber(LinearSolveError, &NumberFormatOptions)));
	}
	if (const float SimulationTime = SimProxy.GetSimulationTime())
	{
		FNumberFormattingOptions NumberFormatOptions;
		NumberFormatOptions.AlwaysSign = false;
		NumberFormatOptions.UseGrouping = false;
		NumberFormatOptions.RoundingMode = ERoundingMode::HalfFromZero;
		NumberFormatOptions.MinimumIntegralDigits = 1;
		NumberFormatOptions.MaximumIntegralDigits = 6;
		NumberFormatOptions.MinimumFractionalDigits = 2;
		NumberFormatOptions.MaximumFractionalDigits = 2;
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("SimulationTime", "Simulation Time: {0}ms"), FText::AsNumber(SimulationTime, &NumberFormatOptions)));
	}
	if (SimProxy.IsTeleported())
	{
		TextValue = ConcatenateLine(TextValue, LOCTEXT("IsTeleported", "Simulation Teleport Activated"));
	}
	return TextValue;
}

DECLARE_DELEGATE_ThreeParams(FClothVisualizationDebugDraw, const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC&, const ::Chaos::FClothVisualizationNoGC::FDrawContext&);

DECLARE_DELEGATE_ThreeParams(FClothVisualizationDebugDrawTexts, const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC&, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext&);

DECLARE_DELEGATE_RetVal_TwoParams(FText, FLocalDebugDisplayString, const FClothEditorSimulationVisualization&, const FClothSimulationProxy& );

DECLARE_DELEGATE_TwoParams(FAdditionalMenus, FClothEditorSimulationVisualization&, FMenuBuilder&);

struct FVisualizationOption
{
	// Actual option entries
	static const FVisualizationOption OptionData[];
	static const uint32 Count;
	static const uint32 SingleVertexOptionIndex = 1;
	static const uint32 FilterSetOptionIndex = 2;

	// Chaos debug draw function
	FClothVisualizationDebugDraw ClothVisualizationDebugDraw;
	FClothVisualizationDebugDrawTexts ClothVisualizationDebugDrawTexts;
	FLocalDebugDisplayString LocalDebugDisplayString;

	// Extra menu building function
	FAdditionalMenus AdditionalMenus;

	FText DisplayName;         // Text for menu entries.
	FText ToolTip;             // Text for menu tooltips.
	bool bDisablesSimulation;  // Whether or not this option requires the simulation to be disabled.
	bool bHidesClothSections;  // Hides the cloth section to avoid zfighting with the debug geometry.
	bool bDefaultFlagValue = false;

	FVisualizationOption(const FClothVisualizationDebugDraw& InClothVisualizationDebugDraw, const FText& InDisplayName, const FText& InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false, bool bInDefaultFlagValue = false, const FAdditionalMenus& InAdditionalMenus = FAdditionalMenus())
		: ClothVisualizationDebugDraw(InClothVisualizationDebugDraw)
		, AdditionalMenus(InAdditionalMenus)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, bDefaultFlagValue(bInDefaultFlagValue)
	{}

	FVisualizationOption(const FLocalDebugDisplayString& InLocalDebugDisplayString, const FText& InDisplayName, const FText& InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false, bool bInDefaultFlagValue = false)
		: LocalDebugDisplayString(InLocalDebugDisplayString)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, bDefaultFlagValue(bInDefaultFlagValue)
	{}

	FVisualizationOption(const FClothVisualizationDebugDrawTexts& InClothVisualizationDebugDrawTexts, const FText & InDisplayName, const FText & InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false, bool bInDefaultFlagValue = false)
		: ClothVisualizationDebugDrawTexts(InClothVisualizationDebugDrawTexts)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, bDefaultFlagValue(bInDefaultFlagValue)
	{}
	
	FVisualizationOption(const FText & InDisplayName, const FText & InToolTip, bool bInDisablesSimulation = false, bool bInHidesClothSections = false, bool bInDefaultFlagValue = false, const FAdditionalMenus& InAdditionalMenus = FAdditionalMenus())
		: AdditionalMenus(InAdditionalMenus)
		, DisplayName(InDisplayName)
		, ToolTip(InToolTip)
		, bDisablesSimulation(bInDisablesSimulation)
		, bHidesClothSections(bInHidesClothSections)
		, bDefaultFlagValue(bInDefaultFlagValue)
	{}

	~FVisualizationOption()
	{
	}
};

const FVisualizationOption FVisualizationOption::OptionData[] =
{
	FVisualizationOption(
		FLocalDebugDisplayString::CreateLambda([](const FClothEditorSimulationVisualization&,const FClothSimulationProxy& Proxy)
		{
			return GetSimulationStatisticsString(Proxy);
		}),
		LOCTEXT("ChaosVisName_SimulationStatistics", "Simulation Statistics"),
		LOCTEXT("ChaosVisName_SimulationStatistics_Tooltip", "Displays simulation statistics"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/true),
	FVisualizationOption(
		LOCTEXT("ChaosVisName_SingleVertex", "Draw Single Vertex"),
		LOCTEXT("ChaosVisName_SingleVertex_Tooltip", "Draw will be filtered to a single vertex"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuSingleVertexSelection(MenuBuilder);
		})),
	FVisualizationOption(
		LOCTEXT("ChaosVisName_FilterSet", "Filter Set"),
		LOCTEXT("ChaosVisName_FilterSet_Tooltip", "Draw will be filtered to a SimVertex set"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuFilterSetSelection(MenuBuilder);
		})),

// DO NOT ADD ABOVE THIS LINE WITHOUT UPDATING FVisualizationOption::SingleVertexOptionIndex and FVisualizationOption::FilterSetOptionIndex

	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawPhysMeshShaded(Context);
		}),
		LOCTEXT("ChaosVisName_PhysMesh", "Physical Mesh (Flat Shaded)"),
		LOCTEXT("ChaosVisName_PhysMeshShaded_ToolTip", "Draws the current physical result as a doubled sided flat shaded mesh"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/true),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawPhysMeshWired(Context);
		}), 
		LOCTEXT("ChaosVisName_PhysMeshWire", "Physical Mesh (Wireframe)"), 
		LOCTEXT("ChaosVisName_PhysMeshWired_ToolTip", "Draws the current physical mesh result in wireframe")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawAnimMeshWired(Context);
		}),
		LOCTEXT("ChaosVisName_AnimMeshWire", "Animated Mesh (Wireframe)"), 
		LOCTEXT("ChaosVisName_AnimMeshWired_ToolTip", "Draws the current animated mesh input in wireframe")),
	FVisualizationOption(
		LOCTEXT("ChaosVisName_HideRenderMesh", "Hide Render Mesh"), 
		LOCTEXT("ChaosVisName_HideRenderMesh_ToolTip", "Hide the render mesh."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/true),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawParticleIndices(Context);
		}),
		LOCTEXT("ChaosVisName_ParticleIndices", "Particle Indices"), 
		LOCTEXT("ChaosVisName_ParticleIndices_ToolTip", "Draws the particle indices as instantiated by the solver")),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawElementIndices(Context);
		}),
		LOCTEXT("ChaosVisName_ElementIndices", "Element Indices"), 
		LOCTEXT("ChaosVisName_ElementIndices_ToolTip", "Draws the element's (triangle or other) indices as instantiated by the solver")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawPointNormals(Context, EditorVisualization.GetPointNormalLength());
		}),
		LOCTEXT("ChaosVisName_PointNormals", "Physical Mesh Normals"), 
		LOCTEXT("ChaosVisName_PointNormals_ToolTip", "Draws the current point normals for the simulation mesh"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false, 
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuPointNormalsLength(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawPointVelocities(Context);
		}),
		LOCTEXT("ChaosVisName_PointVelocities", "Point Velocities"), 
		LOCTEXT("ChaosVisName_PointVelocities_ToolTip", "Draws the current point velocities for the simulation mesh")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawAnimNormals(Context, EditorVisualization.GetAnimatedNormalLength());
		}),
		LOCTEXT("ChaosVisName_AnimNormals", "Animated Mesh Normals"), 
		LOCTEXT("ChaosVisName_AnimNormals_ToolTip", "Draws the current point normals for the animated mesh"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuAnimatedNormalsLength(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			constexpr bool bForceWireframe = false;
			Visualization.DrawCollision(Context, bForceWireframe);
		}),
		LOCTEXT("ChaosVisName_Collision", "Collisions"), 
		LOCTEXT("ChaosVisName_Collision_ToolTip", "Draws the collision bodies the simulation is currently using")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			constexpr bool bForceWireframe = true;
			Visualization.DrawCollision(Context, bForceWireframe);
		}),
		LOCTEXT("ChaosVisName_CollisionWireframe", "Collisions (Force Wireframe)"),
		LOCTEXT("ChaosVisName_Collision_ToolTip", "Draws the collision bodies the simulation is currently using")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawBackstops(Context);
		}),
		LOCTEXT("ChaosVisName_Backstop", "Backstops"), 
		LOCTEXT("ChaosVisName_Backstop_ToolTip", "Draws the backstop radius and position for each simulation particle")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawBackstopDistances(Context);
		}),
		LOCTEXT("ChaosVisName_BackstopDistance", "Backstop Distances"), 
		LOCTEXT("ChaosVisName_BackstopDistance_ToolTip", "Draws the backstop distance offset for each simulation particle"), 
		/*bDisablesSimulation =*/true),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawMaxDistances(Context);
		}),
		LOCTEXT("ChaosVisName_MaxDistance", "Max Distances"), 
		LOCTEXT("ChaosVisName_MaxDistance_ToolTip", "Draws the current max distances for the sim particles as a line along its normal")),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawMaxDistanceValues(Context);
		}),
		LOCTEXT("ChaosVisName_MaxDistanceValue", "Max Distances As Numbers"), 
		LOCTEXT("ChaosVisName_MaxDistanceValue_ToolTip", "Draws the current max distances as numbers")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawAnimDrive(Context);
		}),
		LOCTEXT("ChaosVisName_AnimDrive", "Anim Drive"), 
		LOCTEXT("ChaosVisName_AnimDrive_Tooltip", "Draws the current skinned reference mesh for the simulation which anim drive will attempt to reach if enabled")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawEdgeConstraint(Context);
		}),
		LOCTEXT("ChaosVisName_EdgeConstraint", "Edge Constraint"),
		LOCTEXT("ChaosVisName_EdgeConstraint_Tooltip", "Draws the edge spring constraints")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawBendingConstraint(Context);
		}),
		LOCTEXT("ChaosVisName_BendingConstraint", "Bending Constraint"), 
		LOCTEXT("ChaosVisName_BendingConstraint_Tooltip", "Draws the bending spring constraints")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawLongRangeConstraint(Context);
		}),
		LOCTEXT("ChaosVisName_LongRangeConstraint" , "Long Range Constraint"), 
		LOCTEXT("ChaosVisName_LongRangeConstraint_Tooltip", "Draws the long range attachment constraint distances")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawWindAndPressureForces(Context, EditorVisualization.GetAerodynamicsLengthScale());
		}),
		LOCTEXT("ChaosVisName_WindAndPressureForces", "Wind Aerodynamic And Pressure Forces"), 
		LOCTEXT("ChaosVisName_WindAndPressure_Tooltip", "Draws the Wind drag and lift and pressure forces"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuAerodynamicsLengthScale(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawWindVelocity(Context, EditorVisualization.GetWindVelocityLengthScale());
		}),
		LOCTEXT("ChaosVisName_WindVelocity", "Wind Velocity"), 
		LOCTEXT("ChaosVisName_WindVelocity_Tooltip", "Draws the Wind Velocity vector"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuWindVelocityLength(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawLocalSpace(Context, EditorVisualization.GetLocalSpaceLengthScale());
		}),
		LOCTEXT("ChaosVisName_LocalSpace", "Local Space Reference Bone"), 
		LOCTEXT("ChaosVisName_LocalSpace_Tooltip", "Draws the local space reference bone"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuLocalSpaceLengthScale(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDrawTexts::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawTextsContext& Context)
		{
			Visualization.DrawLocalSpaceBoneNames(Context);
		}),
		LOCTEXT("ChaosVisName_LocalSpaceBoneNames", "Local Space Reference Bone Names"), 
		LOCTEXT("ChaosVisName_LocalSpaceBoneNames_ToolTip", "Draws the local space reference bone names")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawVelocityScale(Context, EditorVisualization.GetVelocityScaleLength());
		}),
		LOCTEXT("ChaosVisName_VelocityScale", "Velocity Scale"), 
		LOCTEXT("ChaosVisName_VelocityScale_Tooltip", "Draws the velocity scale"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuVelocityScaleLength(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawGravity(Context);
		}),
		LOCTEXT("ChaosVisName_Gravity", "Gravity"), 
		LOCTEXT("ChaosVisName_Gravity_Tooltip", "Draws gravity")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawFictitiousAngularForces(Context);
		}),
		LOCTEXT("ChaosVisName_FictitiousAngularForces", "Fictitious Angular Forces"),
		LOCTEXT("ChaosVisName_Gravity_FictitiousAngularForces", "Draws fictitious angular forces (force based solver only)")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawSelfCollision(Context);
		}),
		LOCTEXT("ChaosVisName_SelfCollision", "Self Collision"), 
		LOCTEXT("ChaosVisName_SelfCollision_Tooltip", "Draws the self collision thickness/debugging information")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawSelfIntersection(Context);
		}),
		LOCTEXT("ChaosVisName_SelfIntersection", "Self Intersection"), 
		LOCTEXT("ChaosVisName_SelfIntersection_Tooltip", "Draws the self intersection contour/region information")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawSelfCollisionLayers(Context);
		}),
		LOCTEXT("ChaosVisName_SelfCollisionLayers", "Self Collision Layers"), 
		LOCTEXT("ChaosVisName_SelfCollisionLayers_Tooltip", "Draws the self collision layers"),
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/true),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawSelfCollisionThickness(Context);
		}),
		LOCTEXT("ChaosVisName_SelfCollisionThickness", "Self Collision Thickness"), 
		LOCTEXT("ChaosVisName_SelfCollisionThickness_Tooltip", "Draws the self collision Thickness")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawClothCollisionThickness(Context);
		}),
		LOCTEXT("ChaosVisName_ClothCollisionThickness", "Cloth Collision Thickness"), 
		LOCTEXT("ChaosVisName_ClothCollisionThickness_Tooltip", "Draws the cloth collision Thickness")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawInnerCollisionThickness(Context);
		}),
		LOCTEXT("ChaosVisName_InnerCollisionThickness", "Inner Collision Thickness"), 
		LOCTEXT("ChaosVisName_InnerCollisionThickness_Tooltip", "Draws the inner collision Thickness")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawKinematicColliderShaded(Context);
		}),
		LOCTEXT("ChaosVisName_DrawKinematicColliderShaded", "Draw Kinematic Colliders (Shaded)"), 
		LOCTEXT("ChaosVisName_DrawKinematicColliderShaded_Tooltip", "Draw kinematic cloth colliders with flat shading.")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawKinematicColliderWired(Context);
		}),
		LOCTEXT("ChaosVisName_DrawKinematicColliderWired", "Draw Kinematic Colliders (Wireframe)"), 
		LOCTEXT("ChaosVisName_DrawKinematicColliderWired_Tooltip", "Draw kinematic cloth colliders in wireframe.")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawClothClothConstraints(Context);
		}),
		LOCTEXT("ChaosVisName_DrawClothClothConstraints", "Cloth-Cloth constraints"), 
		LOCTEXT("ChaosVisName_DrawClothClothConstraints_Tooltip", "Draw cloth-cloth constraints.")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawTeleportReset(Context);
		}),
		LOCTEXT("ChaosVisName_DrawTeleportReset", "Teleport/Reset"), 
		LOCTEXT("ChaosVisName_DrawTeleportReset_Tooltip", "Draw teleport/reset status.")),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			const FString* const WeightMap = EditorVisualization.GetCurrentlySelectedWeightMap();
			Visualization.DrawWeightMapWithName(Context, WeightMap ? *WeightMap : FString());
		}),
		LOCTEXT("ChaosVisName_DrawWeightMap", "Weight Map"), 
		LOCTEXT("ChaosVisName_DrawWeightMap_ToolTip", "Draw the weight map for the simulation mesh. You can control the name of the map to be visualized by setting the p.ChaosClothVisualization.WeightMapName console variable."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/true, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuWeightMapSelector(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			const FString* const MorphTargetName = EditorVisualization.GetCurrentlySelectedMorphTarget();
			Visualization.DrawSimMorphTarget(Context, MorphTargetName ? *MorphTargetName : FString());
		}),
		LOCTEXT("ChaosVisName_DrawSimMorphTarget", "Sim Morph Target"), 
		LOCTEXT("ChaosVisName_DrawSimMorphTarget_ToolTip", "Draw a sim morph target. If none selected, the currently active morph target is displayed."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuMorphTargetSelector(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			const FString* const AccessoryMeshName = EditorVisualization.GetCurrentlySelectedAccessoryMesh();
			Visualization.DrawAccessoryMesh(Context, AccessoryMeshName ? FName(*AccessoryMeshName) : NAME_None);
		}),
		LOCTEXT("ChaosVisName_DrawAccessoryMesh", "Accessory Mesh"), 
		LOCTEXT("ChaosVisName_DrawAccessoryMesh_ToolTip", "Draw an accessory mesh."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuAccessoryMeshSelector(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization& EditorVisualization, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			const FString* const AccessoryMeshName = EditorVisualization.GetCurrentlySelectedAccessoryMesh();
			Visualization.DrawAccessoryMeshNormals(Context, AccessoryMeshName ? FName(*AccessoryMeshName) : NAME_None, EditorVisualization.GetAccessoryMeshNormalLength());
		}),
		LOCTEXT("ChaosVisName_DrawAccessoryMeshNormals", "Accessory Mesh Normals"), 
		LOCTEXT("ChaosVisName_DrawAccessoryMeshNormals_ToolTip", "Draw an accessory mesh's normals."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/false, /*bDefaultFlagValue=*/false,
		FAdditionalMenus::CreateLambda([](FClothEditorSimulationVisualization& EditorVisualization, FMenuBuilder& MenuBuilder)
		{
			EditorVisualization.ExtendViewportShowMenuAccessoryMeshNormalsLength(MenuBuilder);
		})),
	FVisualizationOption(
		FClothVisualizationDebugDraw::CreateLambda([](const FClothEditorSimulationVisualization&, const ::Chaos::FClothVisualizationNoGC& Visualization, const ::Chaos::FClothVisualizationNoGC::FDrawContext& Context)
		{
			Visualization.DrawInpaintWeightsMatched(Context);
		}),
		LOCTEXT("ChaosVisName_DrawInpaintWeightsMatched", "Transfer Skin Weights Node: Matched Vertices"), 
		LOCTEXT("ChaosVisName_DrawInpaintWeightsMatched_ToolTip", "When transferring weights using the InpaintWeights method, will highlight the vertices for which we copied the weights directly from the source mesh. For all other vertices, the weights were computed automatically."), 
		/*bDisablesSimulation =*/false, /*bHidesClothSections=*/true)
};
const uint32 FVisualizationOption::Count = sizeof(OptionData) / sizeof(FVisualizationOption);

} // namespace Private

FClothEditorSimulationVisualization::FClothEditorSimulationVisualization()
	: Flags(false, Private::FVisualizationOption::Count)
{
	for (uint32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
	{
		Flags[OptionIndex] = Private::FVisualizationOption::OptionData[OptionIndex].bDefaultFlagValue;
	}
}


void FClothEditorSimulationVisualization::ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, FEditorViewportClient& ViewportClient, const TFunction<UChaosClothComponent*()>& GetClothComponentFunc)
{
	MenuBuilder.BeginSection(TEXT("ChaosSimulation_Visualizations"), LOCTEXT("ClothVisualizationSection", "Chaos Cloth Visualization"));
	{
		for (uint32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
		{
			// Handler for visualization entry being clicked
			const FExecuteAction ExecuteAction = FExecuteAction::CreateLambda([this, OptionIndex, ViewportClientPtr = &ViewportClient, GetClothComponentFunc]()
			{
				Flags[OptionIndex] = !Flags[OptionIndex];

				if (UChaosClothComponent* const ClothComponent = GetClothComponentFunc())
				{
					// If we need to toggle the disabled or visibility states, handle it
					// Disable simulation
					const bool bShouldDisableSimulation = ShouldDisableSimulation();
					if (ClothComponent->IsSimulationEnabled() == bShouldDisableSimulation)
					{
						ClothComponent->SetEnableSimulation(!bShouldDisableSimulation);
					}
					// Hide cloth section
					if (Private::FVisualizationOption::OptionData[OptionIndex].bHidesClothSections)
					{
						const bool bIsClothSectionsVisible = !Flags[OptionIndex];
						ShowClothSections(ClothComponent, bIsClothSectionsVisible);
					}

					// refresh the view
					if (ViewportClientPtr)
					{
						ViewportClientPtr->Invalidate();
					}

				}
			});

			// Checkstate function for visualization entries
			const FIsActionChecked IsActionChecked = FIsActionChecked::CreateLambda([this, OptionIndex]()
			{
				return Flags[OptionIndex];
			});

			const FUIAction Action(ExecuteAction, FCanExecuteAction(), IsActionChecked);

			// Add menu entry
			MenuBuilder.AddMenuEntry(Private::FVisualizationOption::OptionData[OptionIndex].DisplayName, Private::FVisualizationOption::OptionData[OptionIndex].ToolTip, FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::ToggleButton);
			Private::FVisualizationOption::OptionData[OptionIndex].AdditionalMenus.ExecuteIfBound(*this, MenuBuilder);
		}
	}
	MenuBuilder.EndSection();
}


void FClothEditorSimulationVisualization::ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, const TSharedRef<FChaosClothAssetEditor3DViewportClient>& ViewportClient)
{
	TWeakPtr<FChaosClothAssetEditor3DViewportClient> ViewportClientWeakPtr(ViewportClient);

	ExtendViewportShowMenu(MenuBuilder, ViewportClient.Get(), [ViewportClientWeakPtr]() -> UChaosClothComponent*
	{
		if (TSharedPtr<FChaosClothAssetEditor3DViewportClient> ViewportClient = ViewportClientWeakPtr.Pin())
		{
			return ViewportClient->GetPreviewClothComponent();
		}

		return nullptr;
	});
}


void FClothEditorSimulationVisualization::ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, const TSharedRef<FDataflowSimulationViewportClient>& ViewportClient)
{
	TWeakPtr<FDataflowSimulationViewportClient> ViewportClientWeakPtr(ViewportClient);

	ExtendViewportShowMenu(MenuBuilder, ViewportClient.Get(), [ViewportClientWeakPtr]() -> UChaosClothComponent*
	{
		if (TSharedPtr<FDataflowSimulationViewportClient> ViewportClient = ViewportClientWeakPtr.Pin())
		{
			if (TSharedPtr<FDataflowEditorToolkit> Toolkit = ViewportClient->GetDataflowEditorToolkit().Pin())
			{
				if (const TSharedPtr<FDataflowSimulationScene>& SimulationScene = Toolkit->GetSimulationScene())
				{
					if (const TObjectPtr<AActor> PreviewActor = SimulationScene->GetPreviewActor())
					{
						return PreviewActor->GetComponentByClass<UChaosClothComponent>();
					}
				}
			}
		}

		return nullptr;
	});
}


void FClothEditorSimulationVisualization::ExtendViewportShowMenuNameSelector(FMenuBuilder& MenuBuilder, FNameSelectionData& SelectionData, const FText& ToolTipText)
{
	SelectionData.Selector =
		SNew(STextComboBox)
		.OptionsSource(&SelectionData.Names)
		.OnSelectionChanged(STextComboBox::FOnTextSelectionChanged::CreateLambda([&SelectionData](TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
			{
				SelectionData.CurrentlySelectedName = Selection;
			}))
		.InitiallySelectedItem(SelectionData.CurrentlySelectedName);

	MenuBuilder.AddMenuEntry(FUIAction(), SelectionData.Selector.ToSharedRef(),
		NAME_None, ToolTipText, EUserInterfaceActionType::None);
}

template<typename T>
void FClothEditorSimulationVisualization::ExtendViewportShowMenuSpinBox(FMenuBuilder& MenuBuilder, T& Value, const T MinValue, const T MaxValue, const T MinSliderValue, const T MaxSliderValue, const FText& ToolTipText)
{
	TSharedRef<SWidget> SpinBox =
		SNew(SSpinBox<T>)
		.Value(Value)
		.OnValueChanged_Lambda([&Value](T NewValue) { Value = NewValue; })
		.MinValue(MinValue)
		.MaxValue(MaxValue)
		.MinSliderValue(MinSliderValue)
		.MaxSliderValue(MaxSliderValue);

	MenuBuilder.AddMenuEntry(FUIAction(), SpinBox,
		NAME_None, ToolTipText, EUserInterfaceActionType::None);
}

void FClothEditorSimulationVisualization::RefreshMenusForClothComponent(const UChaosClothComponent* ClothComponent)
{
	TArray<FString> WeightMaps;
	TArray<FString> MorphTargets;
	TArray<FString> AccessoryMeshes;
	TArray<FString> SelectionSets;
	if (const ::Chaos::FClothVisualizationNoGC* const Visualization = ClothComponent && ClothComponent->GetClothSimulationProxy() ? ClothComponent->GetClothSimulationProxy()->GetClothVisualization() : nullptr)
	{
		WeightMaps = Visualization->GetAllWeightMapNames();
		MorphTargets = Visualization->GetAllMorphTargetNames();
		SelectionSets = Visualization->GetAllVertexSetNames();
		const TArray<FName> AccessoryMeshNames = Visualization->GetAllAccessoryMeshNames();
		AccessoryMeshes.Reserve(AccessoryMeshNames.Num());
		for (const FName& AccessoryMeshName : AccessoryMeshNames)
		{
			AccessoryMeshes.Add(AccessoryMeshName.ToString());
		}
	}

	auto RefreshMenuNames = [](const TArray<FString>& AllNames, FNameSelectionData& SelectionData)
		{
			bool bFoundCurrentlySelected = false;
			SelectionData.Names.Empty(AllNames.Num());
			for (const FString& Map : AllNames)
			{
				const int32 Index = SelectionData.Names.Emplace(MakeShared<FString>(Map));
				if (!bFoundCurrentlySelected && SelectionData.CurrentlySelectedName && Map == *SelectionData.CurrentlySelectedName)
				{
					SelectionData.CurrentlySelectedName = SelectionData.Names[Index];
					bFoundCurrentlySelected = true;
				}
			}
			if (!bFoundCurrentlySelected)
			{
				if (SelectionData.Names.IsEmpty())
				{
					SelectionData.CurrentlySelectedName.Reset();
				}
				else
				{
					SelectionData.CurrentlySelectedName = SelectionData.Names[0];
				}
			}
			if (SelectionData.Selector)
			{
				SelectionData.Selector->RefreshOptions();
				SelectionData.Selector->SetSelectedItem(SelectionData.CurrentlySelectedName);
			}
		};

	RefreshMenuNames(WeightMaps, WeightMapSelection);
	RefreshMenuNames(MorphTargets, MorphTargetSelection);
	RefreshMenuNames(AccessoryMeshes, AccessoryMeshSelection);
	RefreshMenuNames(SelectionSets, FilterSetSelection);
}

void FClothEditorSimulationVisualization::DebugDrawSimulation(const UChaosClothComponent* ClothComponent, FPrimitiveDrawInterface* PDI)
{
	const ::Chaos::FClothVisualizationNoGC* const Visualization = ClothComponent && ClothComponent->GetClothSimulationProxy() ? ClothComponent->GetClothSimulationProxy()->GetClothVisualization() : nullptr;
	if (!Visualization)
	{
		return;
	}

	::Chaos::FClothVisualizationNoGC::FDrawContext Context;
	Context.PDI = PDI;

	if (Flags[Private::FVisualizationOption::SingleVertexOptionIndex])
	{
		Context.Filter.SingleVertex = GetSingleVertexSelectionValue();
	}
	if (Flags[Private::FVisualizationOption::FilterSetOptionIndex])
	{
		const FString* VertexSet = GetCurrentlySelectedFilterSetSelection();
		Context.Filter.VertexSet = VertexSet ? *VertexSet : FString();
	}

	for (int32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
	{
		if (Flags[OptionIndex])
		{
			Private::FVisualizationOption::OptionData[OptionIndex].ClothVisualizationDebugDraw.ExecuteIfBound(*this, *Visualization, Context);
		}
	}
}

void FClothEditorSimulationVisualization::DebugDrawSimulationTexts(const UChaosClothComponent* ClothComponent, FCanvas* Canvas, const FSceneView* SceneView)
{
	const ::Chaos::FClothVisualizationNoGC* const Visualization = ClothComponent && ClothComponent->GetClothSimulationProxy() ? ClothComponent->GetClothSimulationProxy()->GetClothVisualization() : nullptr;
	if (!Visualization)
	{
		return;
	}
	::Chaos::FClothVisualizationNoGC::FDrawTextsContext Context;
	Context.Canvas = Canvas;
	Context.SceneView = SceneView;

	if (Flags[Private::FVisualizationOption::SingleVertexOptionIndex])
	{
		Context.Filter.SingleVertex = GetSingleVertexSelectionValue();
	}
	if (Flags[Private::FVisualizationOption::FilterSetOptionIndex])
	{
		const FString* VertexSet = GetCurrentlySelectedFilterSetSelection();
		Context.Filter.VertexSet = VertexSet ? *VertexSet : FString();
	}

	for (int32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
	{
		if (Flags[OptionIndex])
		{
			Private::FVisualizationOption::OptionData[OptionIndex].ClothVisualizationDebugDrawTexts.ExecuteIfBound(*this, *Visualization, Context);
		}
	}
}

FText FClothEditorSimulationVisualization::GetDisplayString(const UChaosClothComponent* ClothComponent) const
{
	const FClothSimulationProxy* const SimProxy = ClothComponent ? ClothComponent->GetClothSimulationProxy() : nullptr;
	if (!SimProxy)
	{
		return FText();
	}

	FText DisplayString;
	for (int32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
	{
		if (Private::FVisualizationOption::OptionData[OptionIndex].LocalDebugDisplayString.IsBound() && Flags[OptionIndex])
		{
			DisplayString = Private::ConcatenateLine(DisplayString, Private::FVisualizationOption::OptionData[OptionIndex].LocalDebugDisplayString.Execute(*this, *SimProxy));
		}
	}
	return DisplayString;
}

bool FClothEditorSimulationVisualization::ShouldDisableSimulation() const
{
	for (uint32 OptionIndex = 0; OptionIndex < Private::FVisualizationOption::Count; ++OptionIndex)
	{
		if (Flags[OptionIndex])
		{
			const Private::FVisualizationOption& Data = Private::FVisualizationOption::OptionData[OptionIndex];

			if (Data.bDisablesSimulation)
			{
				return true;
			}
		}
	}
	return false;
}

void FClothEditorSimulationVisualization::ShowClothSections(UChaosClothComponent* ClothComponent, bool bIsClothSectionsVisible) const
{
	if (FSkeletalMeshRenderData* const SkeletalMeshRenderData = ClothComponent->GetSkeletalMeshRenderData())
	{
		for (int32 LODIndex = 0; LODIndex < SkeletalMeshRenderData->LODRenderData.Num(); ++LODIndex)
		{
			FSkeletalMeshLODRenderData& SkeletalMeshLODRenderData = SkeletalMeshRenderData->LODRenderData[LODIndex];

			for (int32 SectionIndex = 0; SectionIndex < SkeletalMeshLODRenderData.RenderSections.Num(); ++SectionIndex)
			{
				FSkelMeshRenderSection& SkelMeshRenderSection = SkeletalMeshLODRenderData.RenderSections[SectionIndex];

				if (SkelMeshRenderSection.HasClothingData())
				{
					ClothComponent->ShowMaterialSection(SkelMeshRenderSection.MaterialIndex, SectionIndex, bIsClothSectionsVisible, LODIndex);
				}
			}
		}
	}
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuWeightMapSelector(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuNameSelector(MenuBuilder, WeightMapSelection, LOCTEXT("WeightMapNameSelection", "Select weight map to draw."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuMorphTargetSelector(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuNameSelector(MenuBuilder, MorphTargetSelection, LOCTEXT("MorphTargetNameSelection", "Select morph target to draw."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuAccessoryMeshSelector(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuNameSelector(MenuBuilder, AccessoryMeshSelection, LOCTEXT("AccessoryMeshNameSelection", "Select accessory mesh to draw."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuFilterSetSelection(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuNameSelector(MenuBuilder, FilterSetSelection, LOCTEXT("FilterSetSelectionName", "Select selection set to filter all debug draw with."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuPointNormalsLength(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, PointNormalLength, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetNormalLength", "Set the normal display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuAnimatedNormalsLength(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, AnimatedNormalLength, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetNormalLength", "Set the normal display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuAerodynamicsLengthScale(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, AerodynamicsLengthScale, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetLengthScale", "Scale the vector display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuAccessoryMeshNormalsLength(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, AccessoryMeshNormalLength, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetNormalLength", "Set the normal display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuWindVelocityLength(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, WindVelocityLengthScale, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetLengthScale", "Scale the vector display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuLocalSpaceLengthScale(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, LocalSpaceLengthScale, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetLengthScale", "Scale the vector display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuVelocityScaleLength(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, VelocityScaleLength, 0.f, FLT_MAX, 0.f, 40.f, LOCTEXT("SetLengthScale", "Scale the vector display length."));
}

void FClothEditorSimulationVisualization::ExtendViewportShowMenuSingleVertexSelection(FMenuBuilder& MenuBuilder)
{
	ExtendViewportShowMenuSpinBox(MenuBuilder, SingleVertexSelection,0, INT_MAX, 0, 1000, LOCTEXT("SetSingleVertex", "Set single vertex debug filter."));
}

} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
