// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/BitArray.h"
#include "Templates/SharedPointer.h"

class FMenuBuilder;
class UChaosClothComponent;
class FPrimitiveDrawInterface;
class FCanvas;
class FEditorViewportClient;
class FSceneView;
class STextComboBox;
class FDataflowSimulationViewportClient;

namespace ESelectInfo
{
enum Type : int;
}

namespace UE::Chaos::ClothAsset
{

class FChaosClothAssetEditor3DViewportClient;

class FClothEditorSimulationVisualization
{
public:
	FClothEditorSimulationVisualization();
	   
	void ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, const TSharedRef<FChaosClothAssetEditor3DViewportClient>& ViewportClient);
	void ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, const TSharedRef<FDataflowSimulationViewportClient>& ViewportClient);

	void DebugDrawSimulation(const UChaosClothComponent* ClothComponent, FPrimitiveDrawInterface* PDI);
	void DebugDrawSimulationTexts(const UChaosClothComponent* ClothComponent, FCanvas* Canvas, const FSceneView* SceneView);
	FText GetDisplayString(const UChaosClothComponent* ClothComponent) const;
	void RefreshMenusForClothComponent(const UChaosClothComponent* ClothComponent);

	// WeightMaps 
	const FString* GetCurrentlySelectedWeightMap() const { return WeightMapSelection.CurrentlySelectedName.Get(); }
	void ExtendViewportShowMenuWeightMapSelector(FMenuBuilder& MenuBuilder);
	// Morph Targets
	const FString* GetCurrentlySelectedMorphTarget() const { return MorphTargetSelection.CurrentlySelectedName.Get(); }
	void ExtendViewportShowMenuMorphTargetSelector(FMenuBuilder& MenuBuilder);
	// Accessory Meshes
	const FString* GetCurrentlySelectedAccessoryMesh() const { return AccessoryMeshSelection.CurrentlySelectedName.Get(); }
	void ExtendViewportShowMenuAccessoryMeshSelector(FMenuBuilder& MenuBuilder);
	// DebugFilterSet
	const FString* GetCurrentlySelectedFilterSetSelection() const { return FilterSetSelection.CurrentlySelectedName.Get(); }
	void ExtendViewportShowMenuFilterSetSelection(FMenuBuilder& MenuBuilder);
	// Normals
	void ExtendViewportShowMenuPointNormalsLength(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuAnimatedNormalsLength(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuAerodynamicsLengthScale(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuAccessoryMeshNormalsLength(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuWindVelocityLength(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuLocalSpaceLengthScale(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuVelocityScaleLength(FMenuBuilder& MenuBuilder);
	void ExtendViewportShowMenuSingleVertexSelection(FMenuBuilder& MenuBuilder);
	float GetPointNormalLength() const { return PointNormalLength; }
	float GetAnimatedNormalLength() const { return AnimatedNormalLength; }
	float GetAccessoryMeshNormalLength() const { return AccessoryMeshNormalLength; }
	float GetAerodynamicsLengthScale() const { return AerodynamicsLengthScale; }
	float GetWindVelocityLengthScale() const { return WindVelocityLengthScale; }
	float GetVelocityScaleLength() const { return VelocityScaleLength; }
	float GetLocalSpaceLengthScale() const { return LocalSpaceLengthScale; }
	int32 GetSingleVertexSelectionValue() const { return SingleVertexSelection; }

private:
	struct FNameSelectionData
	{
		TSharedPtr<STextComboBox> Selector;
		TArray<TSharedPtr<FString>> Names;
		TSharedPtr<FString> CurrentlySelectedName;
	};


	void ExtendViewportShowMenu(FMenuBuilder& MenuBuilder, FEditorViewportClient& ViewportClient, const TFunction<UChaosClothComponent*()>& GetClothComponentFunc);

	/** Return whether or not - given the current enabled options - the simulation should be disabled. */
	bool ShouldDisableSimulation() const;
	/** Show/hide all cloth sections for the specified mesh compoment. */
	void ShowClothSections(UChaosClothComponent* ClothComponent, bool bIsClothSectionsVisible) const;
	template<typename T>
	void ExtendViewportShowMenuSpinBox(FMenuBuilder& MenuBuilder, T& Value, const T MinValue, const T MaxValue, const T MinSliderValue, const T MaxSliderValue, const FText& ToolTipText);
	void ExtendViewportShowMenuNameSelector(FMenuBuilder& MenuBuilder, FNameSelectionData& SelectionData, const FText& ToolTipText);

private:
	/** Flags used to store the checked status for the visualization options. */
	TBitArray<> Flags;
	FNameSelectionData WeightMapSelection;
	FNameSelectionData MorphTargetSelection;
	FNameSelectionData AccessoryMeshSelection;
	FNameSelectionData FilterSetSelection;
	float PointNormalLength = 7.f;
	float AnimatedNormalLength = 7.f;
	float AccessoryMeshNormalLength = 7.f;
	float AerodynamicsLengthScale = 10.f;
	float WindVelocityLengthScale = .1f;
	float VelocityScaleLength = .1f;
	float LocalSpaceLengthScale = 1.f;
	int32 SingleVertexSelection = 0;
};
} // namespace UE::Chaos::ClothAsset
