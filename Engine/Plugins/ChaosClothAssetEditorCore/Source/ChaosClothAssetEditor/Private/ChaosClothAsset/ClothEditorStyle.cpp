// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorStyle.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

namespace UE::Chaos::ClothAsset
{
const FName FChaosClothAssetEditorStyle::StyleName("ClothStyle");

FChaosClothAssetEditorStyle::FChaosClothAssetEditorStyle()
	: FSlateStyleSet(StyleName)
{
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	if (const TSharedPtr<IPlugin> ChaosClothAssetEditorPlugin = IPluginManager::Get().GetModuleOwnerPlugin(UE_MODULE_NAME))
	{
		SetContentRoot(ChaosClothAssetEditorPlugin->GetContentDir());

		// Some standard icon sizes used elsewhere in the editor
		const FVector2D Icon8x8(8.0f, 8.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon28x28(28.0f, 28.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon120(120.0f, 120.0f);

		// Icon sizes used in this style set
		const FVector2D ViewportToolbarIconSize = Icon16x16;
		const FVector2D ToolbarIconSize = Icon20x20;

		FString PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::AddWeightMapNodeIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/PaintMaps", ToolbarIconSize));

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::AddMeshSelectionNodeIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/MeshSelect", ToolbarIconSize));

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::AddTransferSkinWeightsNodeIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/SkinTransfer", ToolbarIconSize));

		//
		// Construction Viewport
		// 
	
		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::ToggleConstructionViewWireframeIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/ClothWireframe_16", ViewportToolbarIconSize));

		//
		// Preview Viewport
		// 

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::TogglePreviewWireframeIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/ClothWireframe_16", ViewportToolbarIconSize));

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::ToggleSimulationSuspendedIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/ClothSimSuspend_16", ViewportToolbarIconSize));

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::SoftResetSimulationIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/ResetSoft_16", ViewportToolbarIconSize));

		PropertyNameString = "ChaosClothAssetEditor." + FChaosClothAssetEditorCommands::HardResetSimulationIdentifier;
		Set(*PropertyNameString, new IMAGE_BRUSH_SVG("Icons/ResetHard_16", ViewportToolbarIconSize));
	}
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FChaosClothAssetEditorStyle::~FChaosClothAssetEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FChaosClothAssetEditorStyle& FChaosClothAssetEditorStyle::Get()
{
	static FChaosClothAssetEditorStyle Inst;
	return Inst;
}
} // namespace UE::Chaos::ClothAsset
