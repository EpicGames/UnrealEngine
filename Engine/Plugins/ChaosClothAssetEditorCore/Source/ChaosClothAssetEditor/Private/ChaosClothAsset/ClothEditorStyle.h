// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

namespace UE::Chaos::ClothAsset
{
/**
 * Slate style set for Cloth Editor
 */
class FChaosClothAssetEditorStyle
	: public FSlateStyleSet
{
public:
	UE_API const static FName StyleName;

	/** Access the singleton instance for this style set */
	static UE_API FChaosClothAssetEditorStyle& Get();

private:
	UE_API FChaosClothAssetEditorStyle();
	UE_API ~FChaosClothAssetEditorStyle();
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
