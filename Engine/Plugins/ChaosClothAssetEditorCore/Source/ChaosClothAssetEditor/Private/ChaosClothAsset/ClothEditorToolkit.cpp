// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorToolkit.h"
#include "ChaosClothAsset/ChaosClothAssetEditorModule.h"
#include "ChaosClothAsset/ClothEditor.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorModeUILayer.h"
#include "ChaosClothAsset/SClothEditor3DViewport.h"
#include "ChaosClothAsset/ClothEditor3DViewportClient.h"
#include "ChaosClothAsset/SClothEditorRestSpaceViewport.h"
#include "ChaosClothAsset/ClothEditorRestSpaceViewportClient.h"
#include "ChaosClothAsset/ClothEditorSimulationVisualization.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothEditorModeToolkit.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothSimulationNodeDetailExtender.h"
#include "ChaosClothAsset/AddWeightMapNode.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/Docking/LayoutExtender.h"
#include "AssetEditorModeManager.h"
#include "AdvancedPreviewScene.h"
#include "SAssetEditorViewport.h"
#include "EditorViewportTabContent.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "ChaosClothAsset/SClothCollectionOutliner.h"
#include "Widgets/Input/SComboBox.h"
#include "Dataflow/DataflowEditorCommands.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowSchema.h"
#include "Dataflow/DataflowEdNode.h"
#include "IStructureDetailsView.h"
#include "Engine/Canvas.h"
#include "PropertyEditorModule.h"
#include "Algo/RemoveIf.h"
#include "AdvancedPreviewSceneModule.h"
#include "Widgets/Layout/SSpacer.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "FileHelpers.h"
#include "UObject/PackageReload.h"
#include "ContextObjectStore.h"
#include "SClothEditorAdvancedPreviewDetailsTab.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetEditorToolkit"

// Same as SDataflowGraphEditor, only adds a ClothAssetEditorToolkit pointer to correctly override the GetDataflowContext() method.
class SClothAssetDataflowGraphEditor : public SDataflowGraphEditor
{
public:
	SLATE_BEGIN_ARGS(SClothAssetDataflowGraphEditor) {}
	SLATE_ARGUMENT_DEFAULT(UEdGraph*, GraphToEdit) = nullptr;
	SLATE_ARGUMENT(FGraphEditorEvents, GraphEvents)
	SLATE_ARGUMENT(TSharedPtr<IStructureDetailsView>, DetailsView)
	SLATE_ARGUMENT(SDataflowGraphEditor::FGraphEvaluationCallback, EvaluateGraph)
	SLATE_ARGUMENT_DEFAULT(UE::Chaos::ClothAsset::FChaosClothAssetEditorToolkit*, ClothAssetEditorToolkit) = nullptr;
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UObject* InAssetOwner)
	{
		check(InArgs._GraphToEdit);
		ClothAssetEditorToolkit = InArgs._ClothAssetEditorToolkit;

		SDataflowGraphEditor::FArguments Arguments;
		Arguments._GraphToEdit = InArgs._GraphToEdit;
		Arguments._GraphEvents = InArgs._GraphEvents;
		Arguments._DetailsView = InArgs._DetailsView;
		Arguments._EvaluateGraph = InArgs._EvaluateGraph;
		Arguments._IsEditable = []()->bool { return true; };
		SDataflowGraphEditor::Construct(Arguments, InAssetOwner);
	}

	//~ Begin FDataflowSNodeInterface interface
	virtual TSharedPtr<UE::Dataflow::FContext> GetDataflowContext() const override
	{
		return ensure(ClothAssetEditorToolkit) ? ClothAssetEditorToolkit->GetDataflowContext() : TSharedPtr<UE::Dataflow::FContext>();
	}

	virtual bool NodesHaveToggleWidget() const override
	{
		return false;
	}

	//~ End FDataflowSNodeInterface interface

private:
	UE::Chaos::ClothAsset::FChaosClothAssetEditorToolkit* ClothAssetEditorToolkit = nullptr;
};

namespace UE::Chaos::ClothAsset
{
const FName FChaosClothAssetEditorToolkit::ClothPreviewTabID(TEXT("ChaosClothAssetEditor_ClothPreviewTab"));
const FName FChaosClothAssetEditorToolkit::OutlinerTabID(TEXT("ChaosClothAssetEditor_OutlinerTab"));
const FName FChaosClothAssetEditorToolkit::PreviewSceneDetailsTabID(TEXT("ChaosClothAssetEditor_PreviewSceneDetailsTab"));
const FName FChaosClothAssetEditorToolkit::GraphCanvasTabId(TEXT("ChaosClothAssetEditor_GraphCanvas"));
const FName FChaosClothAssetEditorToolkit::NodeDetailsTabId(TEXT("ChaosClothAssetEditor_NodeDetails"));
const FName FChaosClothAssetEditorToolkit::SimulationVisualizationTabID(TEXT("ChaosClothAssetEditor_SimulationVisualizationTab"));

FChaosClothAssetEditorToolkit::FChaosClothAssetEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseCharacterFXEditorToolkit(InOwningAssetEditor, FName("ChaosClothAssetEditor"))
{
	check(Cast<UChaosClothAssetEditor>(InOwningAssetEditor));

	// We will replace the StandaloneDefaultLayout that our parent class gave us with 
	// one where the properties detail panel is a vertical column on the left, and there
	// are two viewports on the right.
	// We define explicit ExtensionIds on the stacks to reference them later when the
	// UILayer provides layout extensions. 
	//
	// Note: Changes to the layout should include a increment to the layout's ID, i.e.
	// ChaosClothAssetEditorLayout[X] -> ChaosClothAssetEditorLayout[X+1]. Otherwise, layouts may be messed up
	// without a full reset to layout defaults inside the editor.
	StandaloneDefaultLayout = FTabManager::NewLayout(FName("ChaosClothAssetEditorLayout6"))
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.8f)	// Relative width of (Tools Panel, Construction Viewport, Preview Viewport, Dataflow Graph Editor, Outliner) vs (Asset Details, Preview Scene Details, Dataflow Node Details)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
					->SetSizeCoefficient(0.55f)	// Relative height of (Tools Panel, Construction Viewport, Preview Viewport) vs (Dataflow Graph Editor, Outliner)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.1f)		// Relative width of (Tools Panel) vs (Construction Viewport, Preview Viewport)
						->SetExtensionId(UChaosClothAssetEditorUISubsystem::EditorSidePanelAreaName)
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.45f)		// Relative width of (Construction Viewport) vs (Tools Panel, Preview Viewport)
						->AddTab(ViewportTabID, ETabState::OpenedTab)
						->SetExtensionId("RestSpaceViewportArea")
						->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.45f)		// Relative width of (Preview Viewport) vs (Tools Panel, Construction Viewport)
						->AddTab(ClothPreviewTabID, ETabState::OpenedTab)
						->SetExtensionId("Viewport3DArea")
						->SetHideTabWell(true)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.45f)	// Relative height of (Dataflow Graph Editor, Outliner) vs (Tools Panel, Construction Viewport, Preview Viewport)
					->AddTab(GraphCanvasTabId, ETabState::OpenedTab)
					->AddTab(OutlinerTabID, ETabState::OpenedTab)
					->SetExtensionId("GraphEditorArea")
					->SetHideTabWell(false)
					->SetForegroundTab(GraphCanvasTabId)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.2f)	// Relative width of (Asset Details, Preview Scene Details, Dataflow Node Details) vs (Tools Panel, Construction Viewport, Preview Viewport, Dataflow Graph Editor, Outliner)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.65f)	// Relative height of (Asset Details, Preview Scene Details) vs (Dataflow Node Details)
					->AddTab(DetailsTabID, ETabState::OpenedTab)
					->AddTab(PreviewSceneDetailsTabID, ETabState::OpenedTab)
					->AddTab(SimulationVisualizationTabID, ETabState::OpenedTab)
					->SetExtensionId("DetailsArea")
					->SetHideTabWell(true)
					->SetForegroundTab(DetailsTabID)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)	// Relative height of (Dataflow Node Details) vs (Asset Details, Preview Scene Details)
					->AddTab(NodeDetailsTabId, ETabState::OpenedTab)
					->SetExtensionId("NodeDetailsArea")
					->SetHideTabWell(false)
				)
			)
		);

	// Add any extenders specified by the UISubsystem
	// The extenders provide defined locations for FModeToolkit to attach
	// tool palette tabs and detail panel tabs
	LayoutExtender = MakeShared<FLayoutExtender>();
	FChaosClothAssetEditorModule* Module = &FModuleManager::LoadModuleChecked<FChaosClothAssetEditorModule>("ChaosClothAssetEditor");
	Module->OnRegisterLayoutExtensions().Broadcast(*LayoutExtender);
	StandaloneDefaultLayout->ProcessExtensions(*LayoutExtender);

	FPreviewScene::ConstructionValues PreviewSceneArgs;
	PreviewSceneArgs.bShouldSimulatePhysics = 1;
	PreviewSceneArgs.bCreatePhysicsScene = 1;
	
	ClothPreviewScene = MakeShared<FChaosClothPreviewScene>(PreviewSceneArgs);
	ClothPreviewScene->SetFloorVisibility(false, true);

	// The PreviewScene's ModeManager is used to keep track of selected components
	ClothPreviewEditorModeManager = MakeShared<FAssetEditorModeManager>();
	ClothPreviewEditorModeManager->SetPreviewScene(ClothPreviewScene.Get());
	ClothPreviewScene->SetModeManager(ClothPreviewEditorModeManager);

	ClothEditorSimulationVisualization = MakeShared<FClothEditorSimulationVisualization>();

	ClothPreviewTabContent = MakeShareable(new FEditorViewportTabContent());
	ClothPreviewViewportClient = MakeShared<FChaosClothAssetEditor3DViewportClient>(ClothPreviewEditorModeManager.Get(), ClothPreviewScene, ClothEditorSimulationVisualization);
	ClothPreviewViewportClient->RegisterDelegates();

	ClothPreviewViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		return SAssignNew(PreviewViewportWidget, SChaosClothAssetEditor3DViewport, InArgs)
			.EditorViewportClient(ClothPreviewViewportClient)
			.ToolkitCommandList(GetToolkitCommands().ToSharedPtr());
	};

	// Construction view scene
	ObjectScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues().SetSkyBrightness(0.0f).SetLightBrightness(0.0f));
}

FChaosClothAssetEditorToolkit::~FChaosClothAssetEditorToolkit()
{
	// This code is also called in OnRequestClose. Leaving this here as well in case the toolkit gets destroyed without having OnRequestClose called
	TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode();
	if (SelectedDataflowNode && OnNodeInvalidatedDelegateHandle.IsValid())
	{
		SelectedDataflowNode->GetOnNodeInvalidatedDelegate().Remove(OnNodeInvalidatedDelegateHandle);
	}
	SelectedDataflowNodeGuid.Invalidate();

	if (ClothPreviewViewportClient)
	{
		// Delete the gizmo in the viewport before deleting the EditorModeManager. The Gizmo Manager can get tripped up if it gets deleted
		// while it still has active gizmos.
		ClothPreviewViewportClient->DeleteViewportGizmo();
	}

	// We need to force the cloth editor mode deletion now because otherwise the preview and rest-space worlds
	// will end up getting destroyed before the mode's Exit() function gets to run, and we'll get some
	// warnings when we destroy any mode actors.
	EditorModeManager->DestroyMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId);
}

TSharedPtr<UE::Dataflow::FEngineContext> FChaosClothAssetEditorToolkit::GetDataflowContext() const
{
	return DataflowContext;
}

const UDataflow* FChaosClothAssetEditorToolkit::GetDataflow() const
{
	return GetAsset()->GetDataflow();
}


UDataflow* FChaosClothAssetEditorToolkit::GetDataflow()
{
	return GetAsset()->GetDataflow();
}

//~ Begin FTickableEditorObject overrides

void FChaosClothAssetEditorToolkit::Tick(float DeltaTime)
{
	// Execute commands that required waiting for the next tick to execute (e.g. refresh of the details view)
	TickCommands.Broadcast();
	TickCommands.Clear();

	// Evaluate terminal node if one is assigned
	EvaluateNode(nullptr, nullptr);

	InvalidateViews();
}

TStatId FChaosClothAssetEditorToolkit::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FChaosClothAssetEditorToolkit, STATGROUP_Tickables);
}


//~ End FTickableEditorObject overrides


//~ Begin FBaseCharacterFXEditorToolkit overrides

FEditorModeID FChaosClothAssetEditorToolkit::GetEditorModeId() const
{
	return UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId;
}


void FChaosClothAssetEditorToolkit::InitializeEdMode(UBaseCharacterFXEditorMode* EdMode)
{
	UChaosClothAssetEditorMode* ClothMode = Cast<UChaosClothAssetEditorMode>(EdMode);
	check(ClothMode);

	check(ClothPreviewScene.IsValid());
	ClothMode->SetPreviewScene(ClothPreviewScene.Get());

	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));

	ClothMode->InitializeTargets(ObjectsToEdit);

	if (TSharedPtr<FModeToolkit> ModeToolkit = ClothMode->GetToolkit().Pin())
	{
		FChaosClothAssetEditorModeToolkit* ClothModeToolkit = static_cast<FChaosClothAssetEditorModeToolkit*>(ModeToolkit.Get());
		ClothModeToolkit->SetRestSpaceViewportWidget(RestSpaceViewportWidget);
	}
}

void FChaosClothAssetEditorToolkit::CreateEditorModeUILayer()
{
	TSharedPtr<class IToolkitHost> PinnedToolkitHost = ToolkitHost.Pin();
	check(PinnedToolkitHost.IsValid());
	ModeUILayer = MakeShared<FChaosClothAssetEditorModeUILayer>(PinnedToolkitHost.Get());
}

//~ End FBaseCharacterFXEditorToolkit overrides


//~ Begin FBaseAssetToolkit overrides

void FChaosClothAssetEditorToolkit::CreateWidgets()
{
	FBaseCharacterFXEditorToolkit::CreateWidgets();

	UChaosClothAsset* const ClothAsset = GetAsset();

	if (ClothAsset)
	{
		UDataflow* const Dataflow = ClothAsset->GetDataflow();

		// TODO: Figure out how to create the GraphEditor widgets when the ClothAsset doesn't have a Dataflow property set
		if (Dataflow)
		{
			Dataflow->Schema = UDataflowSchema::StaticClass();

			NodeDetailsEditor = CreateNodeDetailsEditorWidget(ClothAsset);
			GraphEditor = CreateGraphEditorWidget();
		}
	}
}

// Called from FBaseAssetToolkit::CreateWidgets. The delegate call path goes through FAssetEditorToolkit::InitAssetEditor
// and FBaseAssetToolkit::SpawnTab_Viewport.
AssetEditorViewportFactoryFunction FChaosClothAssetEditorToolkit::GetViewportDelegate()
{
	AssetEditorViewportFactoryFunction TempViewportDelegate = [this](FAssetEditorViewportConstructionArgs InArgs)
	{
		return SAssignNew(RestSpaceViewportWidget, SChaosClothAssetEditorRestSpaceViewport, InArgs)
			.RestSpaceViewportClient(StaticCastSharedPtr<FChaosClothEditorRestSpaceViewportClient>(ViewportClient));
	};

	return TempViewportDelegate;
}

// Called from FBaseAssetToolkit::CreateWidgets to populate ViewportClient, but otherwise only used 
// in our own viewport delegate.
TSharedPtr<FEditorViewportClient> FChaosClothAssetEditorToolkit::CreateEditorViewportClient() const
{
	// Note that we can't reliably adjust the viewport client here because we will be passing it
	// into the viewport created by the viewport delegate we get from GetViewportDelegate(), and
	// that delegate may (will) affect the settings based on FAssetEditorViewportConstructionArgs,
	// namely ViewportType.
	// Instead, we do viewport client adjustment in PostInitAssetEditor().
	check(EditorModeManager.IsValid());
	return MakeShared<FChaosClothEditorRestSpaceViewportClient>(EditorModeManager.Get(), ObjectScene.Get());
}

//~ End FBaseAssetToolkit overrides


//~ Begin FAssetEditorToolkit overrides

void FChaosClothAssetEditorToolkit::AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder)
{
	TSharedPtr<SChaosClothAssetEditorRestSpaceViewport> ViewportWidget = StaticCastSharedPtr<SChaosClothAssetEditorRestSpaceViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->AddOverlayWidget(InViewportOverlayWidget, ZOrder);
}

void FChaosClothAssetEditorToolkit::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
{
	TSharedPtr<SChaosClothAssetEditorRestSpaceViewport> ViewportWidget = StaticCastSharedPtr<SChaosClothAssetEditorRestSpaceViewport>(ViewportTabContent->GetFirstViewport());
	ViewportWidget->RemoveOverlayWidget(InViewportOverlayWidget);
}

bool FChaosClothAssetEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	// Note: This needs a bit of adjusting, because currently OnRequestClose seems to be 
	// called multiple times when the editor itself is being closed. We can take the route 
	// of NiagaraScriptToolkit and remember when changes are discarded, but this can cause
	// issues if the editor close sequence is interrupted due to some other asset editor.

	UChaosClothAssetEditorMode* ClothEdMode = Cast<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
	if (!ClothEdMode) {
		// If we don't have a valid mode, because the OnRequestClose is currently being called multiple times,
		// simply return true because there's nothing left to do.
		return true;
	}

	TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode();
	if (SelectedDataflowNode && OnNodeInvalidatedDelegateHandle.IsValid())
	{
		SelectedDataflowNode->GetOnNodeInvalidatedDelegate().Remove(OnNodeInvalidatedDelegateHandle);
	}
	SelectedDataflowNodeGuid.Invalidate();

	FCoreUObjectDelegates::OnPackageReloaded.Remove(OnPackageReloadedDelegateHandle);

	return FAssetEditorToolkit::OnRequestClose(InCloseReason);
}

void FChaosClothAssetEditorToolkit::OnClose()
{
	// Give any active modes a chance to shutdown while the toolkit host is still alive
	// This is super important to do, otherwise currently opened tabs won't be marked as "closed".
	// This results in tabs not being properly recycled upon reopening the editor and tab
	// duplication for each opening event.
	GetEditorModeManager().ActivateDefaultMode();

	FBaseCharacterFXEditorToolkit::OnClose();
}

void FChaosClothAssetEditorToolkit::PostInitAssetEditor()
{
	FBaseCharacterFXEditorToolkit::PostInitAssetEditor();

	// Custom viewport setup

	auto SetCommonViewportClientOptions = [](FEditorViewportClient* Client)
	{
		// Normally the bIsRealtime flag is determined by whether the connection is remote, but our
		// tools require always being ticked.
		Client->SetRealtime(true);

		// Disable motion blur effects that cause our renders to "fade in" as things are moved
		Client->EngineShowFlags.SetTemporalAA(false);
		Client->EngineShowFlags.SetAntiAliasing(true);
		Client->EngineShowFlags.SetMotionBlur(false);

		// Disable the dithering of occluded portions of gizmos.
		Client->EngineShowFlags.SetOpaqueCompositeEditorPrimitives(true);

		// Disable hardware occlusion queries, which make it harder to use vertex shaders to pull materials
		// toward camera for z ordering because non-translucent materials start occluding themselves (once
		// the component bounds are behind the displaced geometry).
		Client->EngineShowFlags.SetDisableOcclusionQueries(true);

		// Default FOV of 90 degrees causes a fair bit of lens distortion, especially noticeable with smaller viewports
		Client->ViewFOV = 45.0;
	};

	{
		// when CreateEditorViewportClient() is called, RestSpaceViewport is null. Set it here instead
		TSharedPtr<FChaosClothEditorRestSpaceViewportClient> VC = StaticCastSharedPtr<FChaosClothEditorRestSpaceViewportClient>(ViewportClient);
		VC->SetEditorViewportWidget(RestSpaceViewportWidget);
	}

	SetCommonViewportClientOptions(ViewportClient.Get());

	// Ortho has too many problems with rendering things, unfortunately, so we should use perspective.
	ViewportClient->SetViewportType(ELevelViewportType::LVT_Perspective);

	// Lit gives us the most options in terms of the materials we can use.
	ViewportClient->SetViewMode(EViewModeIndex::VMI_Lit);

	UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));

	// If exposure isn't set to fixed, it will flash as we stare into the void
	ViewportClient->ExposureSettings.bFixed = true;

	const TWeakPtr<FViewportClient> WeakViewportClient(ViewportClient);
	ClothMode->SetRestSpaceViewportClient(StaticCastWeakPtr<FChaosClothEditorRestSpaceViewportClient>(WeakViewportClient));

	// Note: We force the cloth preview viewport to open, since some ViewportClient functions are not robust to having no viewport.
	// See UE-114649
	if (!TabManager->FindExistingLiveTab(ClothPreviewTabID))
	{
		TabManager->TryInvokeTab(ClothPreviewTabID);
	}

	// We need the viewport client to start out focused, or else it won't get ticked until
	// we click inside it.
	ViewportClient->ReceivedFocus(ViewportClient->Viewport);

	// Set up 3D viewport
	ClothPreviewViewportClient->SetClothEdMode(ClothMode);
	ClothPreviewViewportClient->SetClothEditorToolkit(StaticCastSharedRef<FChaosClothAssetEditorToolkit>(this->AsShared()));

	SetCommonViewportClientOptions(ClothPreviewViewportClient.Get());
	ClothPreviewViewportClient->SetInitialViewTransform(ELevelViewportType::LVT_Perspective, FVector(0, 0, 0), FRotator(0, -90, 0), DEFAULT_ORTHOZOOM);

	if (ClothPreviewViewportClient->Viewport != nullptr)
	{
		FBox PreviewBounds = ClothMode->PreviewBoundingBox();
		ClothPreviewViewportClient->FocusViewportOnBox(PreviewBounds);
	}

	InitDetailsViewPanel();

	ClothMode->DataflowGraph = GetDataflow();
	ClothMode->SetDataflowGraphEditor(GraphEditor);
	DataflowContext = TSharedPtr<UE::Dataflow::FEngineContext>(new UE::Dataflow::FClothAssetDataflowContext(GetAsset(), GetDataflow()));
	ClothMode->SetDataflowContext(DataflowContext);

	// Handle Dataflow asset reload event
	OnPackageReloadedDelegateHandle = FCoreUObjectDelegates::OnPackageReloaded.AddSP(this, &FChaosClothAssetEditorToolkit::HandlePackageReloaded);

	// Populate editor toolbar

	FName ParentToolbarName;
	const FName ToolBarName = GetToolMenuToolbarName(ParentToolbarName);
	UToolMenu* const AssetToolbar = UToolMenus::Get()->ExtendMenu(ToolBarName);
	FToolMenuSection& Section = AssetToolbar->FindOrAddSection("ClothTools");

	for (const TPair<FName, TSharedPtr<const FUICommandInfo>>& NodeAndAddCommand : ClothMode->NodeTypeToAddNodeCommandMap)
	{
		ToolkitCommands->MapAction(NodeAndAddCommand.Value,
			FExecuteAction::CreateUObject(ClothMode, &UChaosClothAssetEditorMode::AddNode, NodeAndAddCommand.Key),
			FCanExecuteAction::CreateUObject(ClothMode, &UChaosClothAssetEditorMode::CanAddNode, NodeAndAddCommand.Key));

		Section.AddEntry(FToolMenuEntry::InitToolBarButton(NodeAndAddCommand.Value));
	}
}

void FChaosClothAssetEditorToolkit::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UAssetEditorToolkitMenuContext* const ClothEditorContext = NewObject<UAssetEditorToolkitMenuContext>();
	ClothEditorContext->Toolkit = SharedThis(this);
	MenuContext.AddObject(ClothEditorContext);
}

void FChaosClothAssetEditorToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	FBaseCharacterFXEditorToolkit::GetSaveableObjects(OutObjects);

	UChaosClothAsset* const ClothAsset = GetAsset();
	UDataflow* const DataflowAsset = ClothAsset->GetDataflow();
	if (DataflowAsset)
	{
		check(DataflowAsset->IsAsset());
		OutObjects.Add(DataflowAsset);
	}
}

void FChaosClothAssetEditorToolkit::OnAssetsSaved(const TArray<UObject*>& SavedObjects)
{
	// Also save the Dataflow asset's dependent objects if required
	TArray<UPackage*> PackagesToSave;

	const UChaosClothAsset* const ClothAsset = GetAsset();
	check(ClothAsset);

	if (const UDataflow* const DataflowAsset = ClothAsset->GetDataflow())
	{
		if (DataflowAsset->Dataflow)
		{
			TArray<UObject*> References;
			FReferenceFinder ReferenceFinder(References);
			DataflowAsset->Dataflow->AddReferencedObjects(ReferenceFinder);

			for (UObject* const Reference : References) //-V1078
			{
				if (Reference->IsAsset())
				{
					PackagesToSave.Add(Reference->GetOutermost());
				}
			}
		}
	}
	constexpr bool bPromptToSave = true;
	constexpr bool bCheckDirtyOnReferenceAssetSave = true;

	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnReferenceAssetSave, bPromptToSave);
}

bool FChaosClothAssetEditorToolkit::ShouldReopenEditorForSavedAsset(const UObject* Asset) const
{
	// "Save As" will potentially save the Dataflow asset with a new name, along with the cloth asset. 
	// We don't really want to open a new Dataflow editor in that case, just the cloth editor
	return Asset->IsA<UChaosClothAsset>();
}

void FChaosClothAssetEditorToolkit::OnAssetsSavedAs(const TArray<UObject*>& SavedObjects)
{
	// Set the Dataflow property on the Cloth object to point to the new DataflowAsset
	UDataflow* NewDataflowAsset = nullptr;
	UChaosClothAsset* NewClothAsset = nullptr;
	for (UObject* const SavedObj : SavedObjects)
	{
		if (SavedObj->IsA<UDataflow>())
		{
			NewDataflowAsset = Cast<UDataflow>(SavedObj);
		}
		else if (SavedObj->IsA<UChaosClothAsset>())
		{
			NewClothAsset = Cast<UChaosClothAsset>(SavedObj);
		}
	}

	if (NewClothAsset && NewDataflowAsset)
	{
		NewClothAsset->SetDataflow(NewDataflowAsset);

		// Now save the new Cloth asset again since we've updated its Property
		const TArray<UPackage*> PackagesToSave{NewClothAsset->GetOutermost()};
		constexpr bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnAssetSave, bPromptToSave);

		// Reload the package so the editor can update anything that references the Dataflow asset
		// (Sadly this function is only called after the editor re-launches as part of the Save As process, so the editor is already open at this point.)
		UPackageTools::ReloadPackages(PackagesToSave);

	}
}

//~ End FAssetEditorToolkit overrides


//~ Begin IToolkit overrides

// This gets used to label the editor's tab in the window that opens.
FText FChaosClothAssetEditorToolkit::GetToolkitName() const
{
	const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();
	if (Objects->Num() == 1)
	{
		return FText::Format(LOCTEXT("ChaosClothAssetEditorTabNameWithObject", "Cloth: {0}"), GetLabelForObject((*Objects)[0]));
	}
	return LOCTEXT("ChaosClothAssetEditorMultipleTabName", "Cloth: Multiple");
}

FName FChaosClothAssetEditorToolkit::GetToolkitFName() const
{
	return FName("Cloth Editor");
}

// Used to create a section in the Help menu for the cloth editor
FText FChaosClothAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ChaosClothAssetEditorBaseName", "Cloth Editor");
}

FText FChaosClothAssetEditorToolkit::GetToolkitToolTipText() const
{
	FString ToolTipString;
	ToolTipString += LOCTEXT("ToolTipAssetLabel", "Asset").ToString();
	ToolTipString += TEXT(": ");

	const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();
	check(Objects && Objects->Num() > 0);
	ToolTipString += GetLabelForObject((*Objects)[0]).ToString();
	for (int32 i = 1; i < Objects->Num(); ++i)
	{
		ToolTipString += TEXT(", ");
		ToolTipString += GetLabelForObject((*Objects)[i]).ToString();
	}

	return FText::FromString(ToolTipString);
}

void FChaosClothAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// We bypass FBaseAssetToolkit::RegisterTabSpawners because it doesn't seem to provide us with
	// anything except tabs that we don't want.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	EditorMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_ChaosClothAssetEditor", "Cloth Editor"));

	// Here we set up the tabs we referenced in StandaloneDefaultLayout (in the constructor).
	// We don't deal with the toolbar palette here, since this is handled by existing
    // infrastructure in FModeToolkit. We only setup spawners for our custom tabs, namely
    // the 2D and 3D viewports, and the details panel.
	InTabManager->RegisterTabSpawner(ClothPreviewTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_ClothPreview))
		.SetDisplayName(LOCTEXT("3DViewportTabLabel", "Cloth 3D Preview Viewport"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(ViewportTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("RestSpaceViewportTabLabel", "Cloth Rest Space Viewport"))
		.SetGroup(EditorMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(SimulationVisualizationTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_SimulationVisualization))
		.SetDisplayName(LOCTEXT("SimulationVisualizationTabDisplayName", "Simulation Visualization"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(OutlinerTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_Outliner))
		.SetDisplayName(LOCTEXT("Outliner", "Outliner"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

	InTabManager->RegisterTabSpawner(PreviewSceneDetailsTabID, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_PreviewSceneDetails))
		.SetDisplayName(LOCTEXT("PreviewSceneDetails", "Preview Scene Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(GraphCanvasTabId, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_GraphCanvas))
		.SetDisplayName(LOCTEXT("DataflowTab", "Graph"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(NodeDetailsTabId, FOnSpawnTab::CreateSP(this, &FChaosClothAssetEditorToolkit::SpawnTab_NodeDetails))
		.SetDisplayName(LOCTEXT("NodeDetailsTab", "Node Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

}

void FChaosClothAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ClothPreviewTabID);
	InTabManager->UnregisterTabSpawner(ViewportTabID);
	InTabManager->UnregisterTabSpawner(DetailsTabID);
	InTabManager->UnregisterTabSpawner(SimulationVisualizationTabID);
	InTabManager->UnregisterTabSpawner(OutlinerTabID);
	InTabManager->UnregisterTabSpawner(PreviewSceneDetailsTabID);
	InTabManager->UnregisterTabSpawner(GraphCanvasTabId);
	InTabManager->UnregisterTabSpawner(NodeDetailsTabId);
}

//~ End IToolkit overrides

void FChaosClothAssetEditorToolkit::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange)
{
	if (UDataflow* const DataflowAsset = GetDataflow())
	{
		FDataflowEditorCommands::OnNotifyPropertyPreChange(NodeDetailsEditor, DataflowAsset, PropertyAboutToChange);
	}
}

UChaosClothAsset* FChaosClothAssetEditorToolkit::GetAsset() const
{
	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));
	
	UObject* ObjectToEdit = nullptr;
	if (ensure(ObjectsToEdit.Num() == 1))
	{
		ObjectToEdit = ObjectsToEdit[0];
	}

	return Cast<UChaosClothAsset>(ObjectToEdit);
}

TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_ClothPreview(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockableTab = SNew(SDockTab);
	const FString LayoutId = FString("ChaosClothAssetEditorClothPreviewViewport");
	ClothPreviewTabContent->Initialize(ClothPreviewViewportDelegate, DockableTab, LayoutId);
	return DockableTab;
}

TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_Outliner(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SAssignNew(Outliner, SClothCollectionOutliner)
		]
	];
	return DockableTab;
}


TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_SimulationVisualization(const FSpawnTabArgs& Args)
{
	SAssignNew(SimulationVisualizationDockTab, SDockTab)
		.Label(LOCTEXT("SimulationVisualizationTitle", "Simulation Visualization"));

	FMenuBuilder MenuBuilder(false, nullptr);
	if (FClothEditorSimulationVisualization* const Visualization = ClothPreviewViewportClient->GetSimulationVisualization().Pin().Get())
	{
		Visualization->ExtendViewportShowMenu(MenuBuilder, ClothPreviewViewportClient.ToSharedRef());
	}
	SimulationVisualizationDockTab->SetContent(MenuBuilder.MakeWidget());

	return SimulationVisualizationDockTab.ToSharedRef();
}

TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_PreviewSceneDetails(const FSpawnTabArgs& Args)
{
	SAssignNew(PreviewSceneDockTab, SDockTab)
		.Label(LOCTEXT("PreviewSceneDetailsTitle", "Preview Scene Details"));

	return PreviewSceneDockTab.ToSharedRef();
}

TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_GraphCanvas(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == GraphCanvasTabId);

	SAssignNew(GraphEditorTab, SDockTab)
		.Label(LOCTEXT("DataflowEditor_Dataflow_TabTitle", "Graph"));

	if (GraphEditor)
	{
		GraphEditorTab.Get()->SetContent(GraphEditor.ToSharedRef());
	}

	return GraphEditorTab.ToSharedRef();
}

TSharedRef<SDockTab> FChaosClothAssetEditorToolkit::SpawnTab_NodeDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == NodeDetailsTabId);

	SAssignNew(NodeDetailsTab, SDockTab)
		.Label(LOCTEXT("DataflowEditor_NodeDetails_TabTitle", "Node Details"));

	if (NodeDetailsEditor)
	{
		NodeDetailsTab.Get()->SetContent(NodeDetailsEditor->GetWidget()->AsShared());
	}

	return NodeDetailsTab.ToSharedRef();
}

void FChaosClothAssetEditorToolkit::InitDetailsViewPanel()
{
	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));

	if (ObjectsToEdit.Num() > 0)
	{
		UObject* const ObjectToEditInDetailsView = ObjectsToEdit[0];
		ensure(ObjectToEditInDetailsView->HasAnyFlags(RF_Transactional));		// Ensure all objects are transactable for undo/redo in the details panel
		SetEditingObject(ObjectToEditInDetailsView);
	}
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FChaosClothAssetEditorToolkit::OnFinishedChangingAssetProperties);

	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;	

	ensure(ClothPreviewScene.IsValid());

	AdvancedPreviewSettingsWidget = SNew(SChaosClothEditorAdvancedPreviewDetailsTab, ClothPreviewScene.ToSharedRef())
		.AdditionalSettings(ClothPreviewScene->GetPreviewSceneDescription())
		.DetailCustomizations(TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>())
		.PropertyTypeCustomizations(TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>())
		.Delegates(Delegates);

	if (PreviewSceneDockTab.IsValid())
	{
		PreviewSceneDockTab->SetContent(AdvancedPreviewSettingsWidget.ToSharedRef());
	}

}


void FChaosClothAssetEditorToolkit::OnFinishedChangingAssetProperties(const FPropertyChangedEvent& Event)
{
	const FProperty* const ChangedProperty = Event.Property;

	if (ChangedProperty && ChangedProperty->GetFName() == TEXT("DataflowAsset"))
	{
		UChaosClothAsset* const ClothAsset = GetAsset();
		if (ClothAsset)
		{
			UDataflow* const Dataflow = ClothAsset->GetDataflow();

			if (Dataflow)
			{
				Dataflow->Schema = UDataflowSchema::StaticClass();
				ReinitializeGraphEditorWidget();
			}
			else
			{
				// Clear the GraphEditor area
				// (Can't have a SDataflowGraphEditor with a null UDataflow, so just put down Spacers if we have no Dataflow)
				GraphEditor.Reset();
				GraphEditorTab.Get()->SetContent(SNew(SSpacer));
				NodeDetailsTab.Get()->SetContent(SNew(SSpacer));
			}

			UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
			ClothMode->DataflowGraph = Dataflow;
			ClothMode->SetDataflowGraphEditor(GraphEditor);
		}
	}
}

void FChaosClothAssetEditorToolkit::EvaluateNode(const FDataflowNode* Node, const FDataflowOutput* Output)
{
	check(DataflowContext);

	UChaosClothAsset* const ClothAsset = GetAsset();
	UDataflow* const Dataflow = GetDataflow(); 

	if (Dataflow && ClothAsset)
	{
		constexpr float NumSteps = 1.f;
		FScopedSlowTask SlowTask(NumSteps, LOCTEXT("ChaosClothAssetEditorToolkitEvaluateNode", "Evaluating nodes..."));
		SlowTask.MakeDialogDelayed(1.f);
		SlowTask.EnterProgressFrame(1.f);

		// Evaluate the node and update LastDataflowNodeTimestamp if doing so
		const UE::Dataflow::FTimestamp OldTimestamp = LastDataflowNodeTimestamp;
		Node = FDataflowEditorCommands::EvaluateNode(
			*DataflowContext,
			LastDataflowNodeTimestamp,
			Dataflow,
			Node,
			Output,
			ClothAsset->GetDataflowInstance().GetDataflowTerminal().ToString(),
			ClothAsset);

		// Refresh editor
		if (Node && OldTimestamp < LastDataflowNodeTimestamp)
		{
			if (Node->GetName() == ClothAsset->GetDataflowInstance().GetDataflowTerminal())
			{
				OnClothAssetChanged();
			}

			// Refresh the construction viewport
			TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode();
			if (Node == SelectedDataflowNode.Get())
			{
				TSharedPtr<FManagedArrayCollection> InputCollection = GetInputClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);
				TSharedPtr<FManagedArrayCollection> Collection = GetClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);
				UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
				ClothMode->SetSelectedClothCollection(Collection, InputCollection);
			}
		}
	}
}

TSharedRef<SDataflowGraphEditor> FChaosClothAssetEditorToolkit::CreateGraphEditorWidget()
{
	UDataflow* const Dataflow = GetDataflow();
	ensure(Dataflow);
	using namespace UE::Dataflow;

	const auto EvalLambda = [this](const FDataflowNode* Node, const FDataflowOutput* Output)
	{
		EvaluateNode(Node, Output);
	};

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeVerifyTitleCommit);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeTitleCommitted);
	InEvents.OnNodeSingleClicked = SGraphEditor::FOnNodeSingleClicked::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeSingleClicked);

	TSharedRef<SDataflowGraphEditor> NewGraphEditor = SNew(SClothAssetDataflowGraphEditor, Dataflow)
		.GraphToEdit(Dataflow)
		.GraphEvents(InEvents)
		.DetailsView(NodeDetailsEditor)
		.EvaluateGraph(EvalLambda)
		.ClothAssetEditorToolkit(this);

	NewGraphEditor->OnSelectionChangedMulticast.AddSP(this, &FChaosClothAssetEditorToolkit::OnNodeSelectionChanged);
	NewGraphEditor->OnNodeDeletedMulticast.AddSP(this, &FChaosClothAssetEditorToolkit::OnNodeDeleted);

	return NewGraphEditor;
}


void FChaosClothAssetEditorToolkit::ReinitializeGraphEditorWidget()
{
	UDataflow* const Dataflow = GetDataflow();

	ensure(Dataflow);

	const auto EvalLambda = [this](const FDataflowNode* Node, const FDataflowOutput* Output)
	{
		EvaluateNode(Node, Output);
	};

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeVerifyTitleCommit);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeTitleCommitted);
	InEvents.OnNodeSingleClicked = SGraphEditor::FOnNodeSingleClicked::CreateSP(this, &FChaosClothAssetEditorToolkit::OnNodeSingleClicked);

	UChaosClothAsset* const ClothAsset = GetAsset();

	if (!GraphEditor)
	{
		NodeDetailsEditor = CreateNodeDetailsEditorWidget(ClothAsset);
		if (NodeDetailsTab.IsValid())
		{
			NodeDetailsTab.Get()->SetContent(NodeDetailsEditor->GetWidget().ToSharedRef());
		}

		GraphEditor = CreateGraphEditorWidget();
		if (GraphEditorTab.IsValid())
		{
			GraphEditorTab.Get()->SetContent(GraphEditor.ToSharedRef());
		}
	}

	SDataflowGraphEditor::FArguments Args;
	Args._GraphToEdit = Dataflow;
	Args._GraphEvents = InEvents;
	Args._DetailsView = NodeDetailsEditor;
	Args._EvaluateGraph = EvalLambda;
	Args._IsEditable = []()->bool { return true; };

	GraphEditor->Construct(Args, ClothAsset);

	GraphEditor->OnSelectionChangedMulticast.RemoveAll(this);
	GraphEditor->OnNodeDeletedMulticast.RemoveAll(this);
	GraphEditor->OnSelectionChangedMulticast.AddSP(this, &FChaosClothAssetEditorToolkit::OnNodeSelectionChanged);
	GraphEditor->OnNodeDeletedMulticast.AddSP(this, &FChaosClothAssetEditorToolkit::OnNodeDeleted);
}

TSharedPtr<IStructureDetailsView> FChaosClothAssetEditorToolkit::CreateNodeDetailsEditorWidget(UObject* ObjectToEdit)
{
	ensure(ObjectToEdit);
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}
	TSharedPtr<IStructureDetailsView> NodeDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	NodeDetailsView->GetDetailsView()->SetObject(ObjectToEdit);
	NodeDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &FChaosClothAssetEditorToolkit::OnPropertyValueChanged);

	NodeDetailsExtender = MakeShared<FClothSimulationNodeDetailExtender>();
	NodeDetailsView->GetDetailsView()->SetExtensionHandler(NodeDetailsExtender);

	return NodeDetailsView;
}

//~ Begin DataflowEditorActions

void FChaosClothAssetEditorToolkit::OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	auto AsEdNodePointers = [](const TSet<UObject*>& Set) {
		TSet<TObjectPtr<UDataflowEdNode>> Nodes;
		for (UObject* Elem : Set)
		{
			if (UDataflowEdNode* Node = Cast<UDataflowEdNode>(Elem))
			{
				Nodes.Add(Node);
			}
		}
		return Nodes;
	};

	FDataflowEditorCommands::OnPropertyValueChanged(GetDataflow(), DataflowContext, LastDataflowNodeTimestamp, PropertyChangedEvent,
		GraphEditor ? AsEdNodePointers(GraphEditor->GetSelectedNodes()) : TSet<TObjectPtr<UDataflowEdNode>>());
}

bool FChaosClothAssetEditorToolkit::OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage) const
{
	return FDataflowEditorCommands::OnNodeVerifyTitleCommit(NewText, GraphNode, OutErrorMessage);
}

void FChaosClothAssetEditorToolkit::OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode) const
{
	FDataflowEditorCommands::OnNodeTitleCommitted(InNewText, InCommitType, GraphNode);
}


TSharedPtr<FManagedArrayCollection> FChaosClothAssetEditorToolkit::GetClothCollectionIfPossible(const TSharedPtr<FDataflowNode> InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
{
	if (InDataflowNode && Context)
	{
		for (const FDataflowOutput* const Output : InDataflowNode->GetOutputs())
		{
			if (Output->GetType() == FName("FManagedArrayCollection"))
			{
				const FManagedArrayCollection DefaultValue;
				TSharedRef<FManagedArrayCollection> Collection = MakeShared<FManagedArrayCollection>(Output->GetValue<FManagedArrayCollection>(*Context, DefaultValue));

				// see if the output collection is a ClothCollection
				const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection);
				if (ClothFacade.IsValid())
				{
					return Collection;
				}

				// The cloth collection schema must be applied to prevent the dynamic mesh conversion and tools from crashing trying to access invalid facades
				break;
			}
		}
	}

	return TSharedPtr<FManagedArrayCollection>();
}


TSharedPtr<FManagedArrayCollection> FChaosClothAssetEditorToolkit::GetInputClothCollectionIfPossible(const TSharedPtr<FDataflowNode> InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context)
{
	if (InDataflowNode && Context)
	{
		for (const FDataflowInput* const Input : InDataflowNode->GetInputs())
		{
			if (Input->GetType() == FName("FManagedArrayCollection"))
			{
				const FManagedArrayCollection DefaultValue;
				TSharedRef<FManagedArrayCollection> Collection = MakeShared<FManagedArrayCollection>(Input->GetValue<FManagedArrayCollection>(*Context, DefaultValue));

				// see if the input collection is a ClothCollection
				const UE::Chaos::ClothAsset::FCollectionClothConstFacade ClothFacade(Collection);
				if (ClothFacade.IsValid())
				{
					return Collection;
				}

				// The cloth collection schema must be applied to prevent the dynamic mesh conversion and tools from crashing trying to access invalid facades
				break;
			}
		}
	}

	return TSharedPtr<FManagedArrayCollection>();
}

TSharedPtr<FDataflowNode> FChaosClothAssetEditorToolkit::GetSelectedDataflowNode()
{
	if (SelectedDataflowNodeGuid.IsValid())
	{
		if (UDataflow* const Dataflow = GetDataflow())
		{
			if (TSharedPtr<UE::Dataflow::FGraph> Graph = Dataflow->GetDataflow())
			{
				return Graph->FindBaseNode(SelectedDataflowNodeGuid);
			}
		}
	}
	return TSharedPtr<FDataflowNode>(nullptr);
}

TSharedPtr<const FDataflowNode> FChaosClothAssetEditorToolkit::GetSelectedDataflowNode() const
{
	if (SelectedDataflowNodeGuid.IsValid())
	{
		if (const UDataflow* const Dataflow = GetDataflow())
		{
			if (TSharedPtr<const UE::Dataflow::FGraph> Graph = Dataflow->GetDataflow())
			{
				return Graph->FindBaseNode(SelectedDataflowNodeGuid);
			}
		}
	}
	return TSharedPtr<const FDataflowNode>(nullptr);
}

void FChaosClothAssetEditorToolkit::OnNodeSelectionChanged(const TSet<UObject*>& NewSelection)
{
	TSharedPtr<FManagedArrayCollection> Collection = nullptr;
	TSharedPtr<FManagedArrayCollection> InputCollection = nullptr;

	// Get any selected node with a ClothCollection output
	// Also, set the selected node(s) to be the Dataflow's RenderTargets
	// TODO: decide if we want selection to be the mechanism for toggling DataflowComponent rendering, or the switch on the Node

	// Despite this function's name, we might not have actually changed which node is selected
	bool bNodeSelectionChanged = false;

	UDataflow* const Dataflow = GetDataflow();
	
	if (Dataflow)
	{
		Dataflow->RenderTargets.Reset();

		if (!NewSelection.Num())
		{
			// No new node selected

			if (TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode())
			{
				bNodeSelectionChanged = true;		// current node was deselected

				if (OnNodeInvalidatedDelegateHandle.IsValid())
				{
					SelectedDataflowNode->GetOnNodeInvalidatedDelegate().Remove(OnNodeInvalidatedDelegateHandle);
				}
				SelectedDataflowNodeGuid.Invalidate();
			}
		}
		else
		{
			for (UObject* const Selected : NewSelection)
			{
				if (UDataflowEdNode* const Node = Cast<UDataflowEdNode>(Selected))
				{
					Dataflow->RenderTargets.Add(Node);

					TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode();
					if (SelectedDataflowNode != Node->GetDataflowNode())
					{
						bNodeSelectionChanged = true;	// changing from one selected node to a different one, or from unselected to selected

						if (SelectedDataflowNode)
						{
							if (OnNodeInvalidatedDelegateHandle.IsValid())
							{
								SelectedDataflowNode->GetOnNodeInvalidatedDelegate().Remove(OnNodeInvalidatedDelegateHandle);
							}
						}
						SelectedDataflowNode = Node->GetDataflowNode();
						SelectedDataflowNodeGuid = SelectedDataflowNode ? SelectedDataflowNode->GetGuid() : FGuid();

						if (SelectedDataflowNode)
						{
							InputCollection = GetInputClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);
							Collection = GetClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);

							// Set a callback to re-evaluate the node if it is invalidated
							OnNodeInvalidatedDelegateHandle = SelectedDataflowNode->GetOnNodeInvalidatedDelegate().AddLambda(
								[this](FDataflowNode* InDataflowNode)
								{
									// Warning: Do not execute code that rebuilds the UI in this lambda as it is called by the UI!

									const TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode();
									if (SelectedDataflowNode.Get() == InDataflowNode)
									{
										// Node was invalidated, update the Construction viewport
										const TSharedPtr<FManagedArrayCollection> InputCollection = GetInputClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);
										const TSharedPtr<FManagedArrayCollection> Collection = GetClothCollectionIfPossible(SelectedDataflowNode, DataflowContext);
										if (UChaosClothAssetEditorMode* const ClothMode =
											CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId)))
										{
											constexpr bool bDeferDynamicMeshInit = false;
											ClothMode->SetSelectedClothCollection(Collection, InputCollection, bDeferDynamicMeshInit);
										}
									}
									// The detail panel won't update correctly when a change affects a node's properties and needs a refresh at the next tick
									TickCommands.AddLambda([this]()
										{
											if (NodeDetailsEditor && NodeDetailsEditor->GetDetailsView())
											{
												NodeDetailsEditor->GetDetailsView()->InvalidateCachedState();
											}
										});

									// The node has just been invalidated, best to close the current tool
									if (const UChaosClothAssetEditorMode* const ClothMode = 
										CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId)))
									{
										UEditorInteractiveToolsContext* const ToolsContext = ClothMode->GetInteractiveToolsContext();
										checkf(ToolsContext, TEXT("No valid ToolsContext found for FChaosClothAssetEditorToolkit"));
										if (ToolsContext->HasActiveTool())
										{
											ToolsContext->EndTool(ToolsContext->CanCancelActiveTool() ? EToolShutdownType::Cancel : EToolShutdownType::Completed);
										}
									}
								});
						}
					}
					break;
				}
			}
		}

		Dataflow->LastModifiedRenderTarget = UE::Dataflow::FTimestamp::Current();
	}


	if (bNodeSelectionChanged)
	{
		UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
		if (ClothMode)
		{
			// Close any running tool. OnNodeSingleClicked() will start a new tool if a new node was clicked.
			UEditorInteractiveToolsContext* const ToolsContext = ClothMode->GetInteractiveToolsContext();
			checkf(ToolsContext, TEXT("No valid ToolsContext found for FChaosClothAssetEditorToolkit"));
			if (ToolsContext->HasActiveTool())
			{
				ToolsContext->EndTool(EToolShutdownType::Completed);
			}

			// Update the Construction viewport with the newly selected node's Collection
			// Defer updating the dynamic mesh component when single selecting since tools want the dynamic mesh collection 
			//   to be built from the InputCollection rather than the (Output)Collection.
			const bool bDeferDynamicMeshInitForTool = NewSelection.Num() == 1;
			ClothMode->SetSelectedClothCollection(Collection, InputCollection, bDeferDynamicMeshInitForTool);

			// Update selected node in Context Object
			if (UDataflowContextObject* DataflowContextObject = ToolsContext->ContextObjectStore->FindContext<UDataflowContextObject>())
			{
				DataflowContextObject->SetSelectedNode(nullptr);

				if (TSharedPtr<FDataflowNode> SelectedDataflowNode = GetSelectedDataflowNode())
				{
					for (UObject* const Selected : NewSelection)
					{
						if (UDataflowEdNode* const Node = Cast<UDataflowEdNode>(Selected))
						{
							if (const TSharedPtr<const FDataflowNode> DataflowNode = Node->GetDataflowNode())
							{
								if (DataflowNode->GetGuid() == SelectedDataflowNode->GetGuid())
								{
									DataflowContextObject->SetSelectedNode(Node);
								}
							}
						}
					}
				}
			}
		}

		if (Outliner)
		{
			Outliner->SetClothCollection(Collection);
		}
	}
}

void FChaosClothAssetEditorToolkit::OnNodeSingleClicked(UObject* ClickedNode) const
{
	UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
	if (ClothMode)
	{
		if (GraphEditor && GraphEditor->GetSingleSelectedNode() == ClickedNode)
		{
			// Start the corresponding tool
			ClothMode->StartToolForSelectedNode(ClickedNode);
		}
	}
}


void FChaosClothAssetEditorToolkit::OnNodeDeleted(const TSet<UObject*>& DeletedNodes)
{
	UDataflow* const Dataflow = GetDataflow();
	if (Dataflow)
	{
		Dataflow->RenderTargets.SetNum(Algo::RemoveIf(Dataflow->RenderTargets, [&DeletedNodes](const UDataflowEdNode* RenderTarget)
		{
			return DeletedNodes.Contains(RenderTarget);
		}));
	}


	UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
	if (ClothMode)
	{
		ClothMode->SetSelectedClothCollection(nullptr);
		ClothMode->OnDataflowNodeDeleted(DeletedNodes);
	}

}


//~ Ends DataflowEditorActions

void FChaosClothAssetEditorToolkit::OnClothAssetChanged()
{
	TArray<TObjectPtr<UObject>> ObjectsToEdit;
	OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsToEdit));

	UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));

	const bool bWasSimulationSuspended = ClothMode->IsSimulationSuspended();

	ClothMode->InitializeTargets(ObjectsToEdit);

	if (UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(ObjectsToEdit[0]))
	{
		const bool bHadClothAsset = (ClothPreviewScene->GetClothComponent()->GetAsset() != nullptr);

		ClothPreviewScene->SetClothAsset(ClothAsset);

		ensure(ClothAsset->HasAnyFlags(RF_Transactional));		// Ensure all objects are transactable for undo/redo in the details panel
		SetEditingObject(ClothAsset);

		if (!bHadClothAsset)
		{
			// Focus on the cloth component if this is the first time adding one
			ClothPreviewViewportClient->FocusViewportOnBox(ClothMode->PreviewBoundingBox());
		}
	}

	if (ClothEditorSimulationVisualization)
	{
		ClothEditorSimulationVisualization->RefreshMenusForClothComponent(ClothPreviewScene->GetClothComponent());
	}

	if (bWasSimulationSuspended)
	{
		ClothMode->SuspendSimulation();
	}
	else
	{
		ClothMode->ResumeSimulation();
	}
}

void FChaosClothAssetEditorToolkit::InvalidateViews()
{
	ViewportClient->Invalidate();
	ClothPreviewViewportClient->Invalidate();
}

void FChaosClothAssetEditorToolkit::HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	// Handle the Dataflow asset being reloaded
	// 
	// UAssetEditorSubsystem::HandlePackageReloaded deals with restarting the corresponding asset editor when an asset is reloaded. However in our case we have the Dataflow Asset 
	// open inside the Cloth Asset Editor and the AssetEditorSubsystem is not tracking that information. So we will handle it ourselves here. 
	// 
	// We will not restart the entire Cloth Editor in this case, but we will rebuild the Graph Editor Widget.
	// 
	// Note we don't need to update the Dataflow pointer in the ClothAsset as the reload process does that -- we just need to know when that pointer changes.

	// TODO: Invesigate if we can somehow tell UAssetEditorSubsystem that when the DataflowAsset is reloaded we should restart the Cloth Editor

	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageFixup)	  // Listen for pre-reload event
	{
		checkf(InPackageReloadedEvent, TEXT("Expected a FPackageReloadedEvent object on PrePackageFixup phase"));

		for (const TPair<UObject*, UObject*>& RepointPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (RepointPair.Key == GetDataflow())
			{
				// Close any open tool
				UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
				UInteractiveToolManager* const ToolManager = ClothMode->GetToolManager();
				if (UInteractiveTool* const ActiveTool = ToolManager->GetActiveTool(EToolSide::Left))
				{
					ToolManager->PostActiveToolShutdownRequest(ActiveTool, EToolShutdownType::Accept);
				}

				// Eliminate anything could be holding a reference to the Dataflow object that will be reloaded (including references to nodes)
				SelectedDataflowNodeGuid.Invalidate();
				GraphEditor.Reset();
				GraphEditorTab.Get()->SetContent(SNew(SSpacer));
			}
		}
	}
	else if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)	  // Listen for post-reload event
	{
		checkf(InPackageReloadedEvent, TEXT("Expected a FPackageReloadedEvent object on PostPackageFixup phase"));

		for (const TPair<UObject*, UObject*>& RepointPair : InPackageReloadedEvent->GetRepointedObjects())
		{
			if (RepointPair.Key == GetDataflow())	// On PostPackageFixup, both Key and Value point to the new Object
			{
				ReinitializeGraphEditorWidget();

				UChaosClothAssetEditorMode* const ClothMode = CastChecked<UChaosClothAssetEditorMode>(EditorModeManager->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
				ClothMode->DataflowGraph = GetDataflow();
				ClothMode->SetDataflowGraphEditor(GraphEditor);
			}
		}
	}
}


} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
