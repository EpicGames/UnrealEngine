// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorToolkit.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

template<typename T> class SComboBox;
class SClothCollectionOutliner;
class UDataflow;
class UChaosClothAsset;
class SGraphEditor;
class SDataflowGraphEditor;
class IStructureDetailsView;
class UEdGraphNode;
class UChaosClothComponent;
class SChaosClothAssetEditorRestSpaceViewport;
class SChaosClothAssetEditor3DViewport;

namespace UE::Chaos::ClothAsset
{
class FClothEditorSimulationVisualization;
class FChaosClothAssetEditor3DViewportClient;
struct FClothSimulationNodeDetailExtender;
}

namespace UE::Dataflow
{
	class FClothAssetDataflowContext final : public TEngineContext<FContextSingle>
	{
	public:
		DATAFLOW_CONTEXT_INTERNAL(TEngineContext<FContextSingle>, FClothAssetDataflowContext);

		FClothAssetDataflowContext(UObject* InOwner, UDataflow* InGraph)
			: Super(InOwner)
		{}
	};
}

namespace UE::Chaos::ClothAsset
{
/**
 * The toolkit is supposed to act as the UI manager for the asset editor. It's responsible
 * for setting up viewports and most toolbars, except for the internals of the mode panel.
 * However, because the toolkit also sets up the mode manager, and much of the important
 * state is held in the UChaosClothAssetEditorMode managed by the mode manager, the toolkit also ends up
 * initializing the Cloth mode.
 * Thus, the FChaosClothAssetEditorToolkit ends up being the central place for the Cloth Asset Editor setup.
 */
class FChaosClothAssetEditorToolkit final : public FBaseCharacterFXEditorToolkit, public FTickableEditorObject, public FNotifyHook
{
public:

	UE_API explicit FChaosClothAssetEditorToolkit(UAssetEditor* InOwningAssetEditor);
	UE_API virtual ~FChaosClothAssetEditorToolkit();

	UE_API TSharedPtr<UE::Dataflow::FEngineContext> GetDataflowContext() const;
	UE_API const UDataflow* GetDataflow() const;

private:

	static UE_API const FName ClothPreviewTabID;
	static UE_API const FName OutlinerTabID;
	static UE_API const FName PreviewSceneDetailsTabID;
	static UE_API const FName SimulationVisualizationTabID;

	// FTickableEditorObject
	UE_API virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	UE_API virtual TStatId GetStatId() const override;

	// FBaseCharacterFXEditorToolkit
	UE_API virtual FEditorModeID GetEditorModeId() const override;
	UE_API virtual void InitializeEdMode(UBaseCharacterFXEditorMode* EdMode) override;
	UE_API virtual void CreateEditorModeUILayer() override;

	// FBaseAssetToolkit
	UE_API virtual void CreateWidgets() override;
	UE_API virtual AssetEditorViewportFactoryFunction GetViewportDelegate() override;
	UE_API virtual TSharedPtr<FEditorViewportClient> CreateEditorViewportClient() const override;

	// FAssetEditorToolkit
	UE_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override;
	UE_API virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;
	UE_API virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void OnClose() override;

	UE_API virtual void PostInitAssetEditor() override;
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	UE_API virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	UE_API virtual bool ShouldReopenEditorForSavedAsset(const UObject* Asset) const override;
	UE_API virtual void OnAssetsSaved(const TArray<UObject*>& SavedObjects) override;
	UE_API virtual void OnAssetsSavedAs(const TArray<UObject*>& SavedObjects) override;

	// IAssetEditorInstance
	virtual bool IsPrimaryEditor() const override { return true; };

	// IToolkit
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;

	// FNotifyHook
	UE_API virtual void NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange) override;

	// Return the cloth asset held by the Cloth Editor
	UE_API UChaosClothAsset* GetAsset() const;
	UE_API UDataflow* GetDataflow();
	
	UE_API TSharedRef<SDockTab> SpawnTab_ClothPreview(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_Outliner(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_PreviewSceneDetails(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_GraphCanvas(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_NodeDetails(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_SimulationVisualization(const FSpawnTabArgs& Args);

	UE_API void InitDetailsViewPanel();
	UE_API void OnFinishedChangingAssetProperties(const FPropertyChangedEvent&);

	UE_API void OnClothAssetChanged();
	UE_API void InvalidateViews();

	// Dataflow
	UE_API void EvaluateNode(const FDataflowNode* Node, const FDataflowOutput* Output);
	UE_API TSharedRef<SDataflowGraphEditor> CreateGraphEditorWidget();
	UE_API void ReinitializeGraphEditorWidget();
	UE_API TSharedPtr<IStructureDetailsView> CreateNodeDetailsEditorWidget(UObject* ObjectToEdit);

	UE_API TSharedPtr<FManagedArrayCollection> GetClothCollectionIfPossible(const TSharedPtr<FDataflowNode> InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context);
	UE_API TSharedPtr<FManagedArrayCollection> GetInputClothCollectionIfPossible(const TSharedPtr<FDataflowNode> InDataflowNode, const TSharedPtr<UE::Dataflow::FEngineContext> Context);
	UE_API TSharedPtr<FDataflowNode> GetSelectedDataflowNode();
	UE_API TSharedPtr<const FDataflowNode> GetSelectedDataflowNode() const;

	// DataflowEditorActions
	UE_API void OnPropertyValueChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API bool OnNodeVerifyTitleCommit(const FText& NewText, UEdGraphNode* GraphNode, FText& OutErrorMessage) const;
	UE_API void OnNodeTitleCommitted(const FText& InNewText, ETextCommit::Type InCommitType, UEdGraphNode* GraphNode) const;
	UE_API void OnNodeSelectionChanged(const TSet<UObject*>& NewSelection);
	UE_API void OnNodeDeleted(const TSet<UObject*>& DeletedNodes);
	UE_API void OnNodeSingleClicked(UObject* ClickedNode) const;

	// Delegates
	UE_API void HandlePackageReloaded(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Scene in which the 3D sim space preview meshes live. Ownership shared with AdvancedPreviewSettingsWidget*/
	TSharedPtr<FChaosClothPreviewScene> ClothPreviewScene;

	TSharedPtr<class FEditorViewportTabContent> ClothPreviewTabContent;
	AssetEditorViewportFactoryFunction ClothPreviewViewportDelegate;
	TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothPreviewViewportClient;
	TSharedPtr<FAssetEditorModeManager> ClothPreviewEditorModeManager;
	TSharedPtr<FClothEditorSimulationVisualization> ClothEditorSimulationVisualization;

	TSharedPtr<SChaosClothAssetEditorRestSpaceViewport> RestSpaceViewportWidget;
	TSharedPtr<SChaosClothAssetEditor3DViewport> PreviewViewportWidget;

	TSharedPtr<SDockTab> PreviewSceneDockTab;
	TSharedPtr<SWidget> AdvancedPreviewSettingsWidget;

	TSharedPtr<SDockTab> SimulationVisualizationDockTab;

	TSharedPtr<SClothCollectionOutliner> Outliner;

	// Dataflow
	TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext;
	UE::Dataflow::FTimestamp LastDataflowNodeTimestamp = UE::Dataflow::FTimestamp::Invalid;
	FDelegateHandle OnNodeInvalidatedDelegateHandle;
	FGuid SelectedDataflowNodeGuid;

	static UE_API const FName GraphCanvasTabId;
	TSharedPtr<SDockTab> GraphEditorTab;
	TSharedPtr<SDataflowGraphEditor> GraphEditor;

	static UE_API const FName NodeDetailsTabId;
	TSharedPtr<SDockTab> NodeDetailsTab;
	TSharedPtr<IStructureDetailsView> NodeDetailsEditor;
	TSharedPtr<FClothSimulationNodeDetailExtender> NodeDetailsExtender;

	FDelegateHandle OnPackageReloadedDelegateHandle;

	DECLARE_MULTICAST_DELEGATE(FTickCommands)
	FTickCommands TickCommands;
};
} // namespace UE::Chaos::ClothAsset

#undef UE_API
