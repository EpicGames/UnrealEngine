// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothEditorViewportToolbarSections.h"

#include "ClothEditorRestSpaceViewportClient.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "EditorModeManager.h"
#include "SEditorViewport.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "ClothEditorViewportToolbarSections"

namespace UE::Chaos::ClothAsset
{

FToolMenuEntry CreateDynamicLightIntensityItem()
{
	return FToolMenuEntry::InitDynamicEntry("DynamicLightIntensity", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
	{
		UUnrealEdViewportToolbarContext* Context = Section.FindContext<UUnrealEdViewportToolbarContext>();
		if (!Context)
		{
			return;
		}
		
		TSharedPtr<SEditorViewport> Viewport = Context->Viewport.Pin();
		if (!Viewport)
		{
			return;
		}
		
		TSharedPtr<FChaosClothEditorRestSpaceViewportClient> ViewportClient = StaticCastSharedPtr<FChaosClothEditorRestSpaceViewportClient>(Viewport->GetViewportClient());
		
		constexpr float IntensityMin = 0.f;
		constexpr float IntensityMax = 20.f;
		
		TSharedRef<SWidget> Widget = SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SSpinBox<float>)
						.Style(&FAppStyle::Get(), "Menu.SpinBox")
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.MinValue(IntensityMin)
						.MaxValue(IntensityMax)
						.Value_Lambda([WeakClient = ViewportClient.ToWeakPtr()]
						{
							if (TSharedPtr<FChaosClothEditorRestSpaceViewportClient> Client = WeakClient.Pin())
							{
								return Client->GetCameraPointLightIntensity();
							}
							return 0.f;
						})
						.OnValueChanged_Lambda([WeakClient = ViewportClient.ToWeakPtr()](float Intensity)
						{
							if (TSharedPtr<FChaosClothEditorRestSpaceViewportClient> Client = WeakClient.Pin())
							{
								Client->SetCameraPointLightIntensity(Intensity);
							}
						}) 
						.IsEnabled_Lambda([WeakClient = ViewportClient.ToWeakPtr()]
						{
							if (TSharedPtr<FChaosClothEditorRestSpaceViewportClient> Client = WeakClient.Pin())
							{
								return Client->GetConstructionViewMode() == EClothPatternVertexType::Render;
							}
							return false;
						})
					]
				]
			];
			
		Section.AddEntry(FToolMenuEntry::InitWidget("LightIntensity", Widget, LOCTEXT("LightIntensityLabel", "Render Light Intensity")));
	}));
}

FToolMenuEntry CreateDynamicSimulationMenuItem()
{
	return FToolMenuEntry::InitDynamicEntry("DynamicSimulationMenu", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
	{
		UUnrealEdViewportToolbarContext* Context = Section.FindContext<UUnrealEdViewportToolbarContext>();
		if (!Context)
		{
			return;
		}
		
		const TAttribute<FText> SimLabel = TAttribute<FText>::CreateLambda([WeakViewport = Context->Viewport]
		{
			if (TSharedPtr<SEditorViewport> Viewport = WeakViewport.Pin())
			{
				const TSharedPtr<FEditorViewportClient> ViewportClient = Viewport->GetViewportClient();
				check(ViewportClient.IsValid());
				const FEditorModeTools* const EditorModeTools = ViewportClient->GetModeTools();
				UChaosClothAssetEditorMode* const ClothEdMode = Cast<UChaosClothAssetEditorMode>(EditorModeTools->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));
				if (ClothEdMode)
				{
					switch (ClothEdMode->GetConstructionViewMode())
					{
					case EClothPatternVertexType::Sim2D:
						return LOCTEXT("ConstructionViewMenuTitle_Sim2D", "2D Sim");
					case EClothPatternVertexType::Sim3D:
						return LOCTEXT("ConstructionViewMenuTitle_Sim3D", "3D Sim");
					case EClothPatternVertexType::Render:
						return LOCTEXT("ConstructionViewMenuTitle_Render", "Render");
					}
				}
			}
			
			return LOCTEXT("ConstructionViewMenuTitle_Default", "View"); 
		});
		
		Section.AddSubMenu(
			"SimMode",
			SimLabel,
			LOCTEXT("SimulationMenuTooltip", "Change the simulation view."),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				FToolMenuSection& Section = InMenu->FindOrAddSection("ConstructionViewModeMenuSection");
				Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().SetConstructionMode2D);
				Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().SetConstructionMode3D);
				Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().SetConstructionModeRender);
			})
		);
	}));
}



}

#undef LOCTEXT_NAMESPACE