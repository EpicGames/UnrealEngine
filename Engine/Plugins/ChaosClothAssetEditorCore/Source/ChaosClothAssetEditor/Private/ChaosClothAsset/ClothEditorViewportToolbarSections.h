// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolMenus.h"

// Common toolbar functions shared by the various cloth editor viewport toolbars.
namespace UE::Chaos::ClothAsset
{
	FToolMenuEntry CreateDynamicLightIntensityItem();
	FToolMenuEntry CreateDynamicSimulationMenuItem();
}
