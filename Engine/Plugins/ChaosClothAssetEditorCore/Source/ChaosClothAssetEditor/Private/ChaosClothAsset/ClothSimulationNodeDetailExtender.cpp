// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothSimulationNodeDetailExtender.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetSimulationNodeDetailExtender"

namespace UE::Chaos::ClothAsset
{
	const FName FClothSimulationNodeDetailExtender::Name = FName("FClothSimulationNodeDetailExtender");

	FName FClothSimulationNodeDetailExtender::GetName() const
	{
		return Name;
	}

	bool FClothSimulationNodeDetailExtender::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
	{
		return PropertyHandle.HasMetaData(TEXT("InteractorName"));
	}

	void FClothSimulationNodeDetailExtender::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		if (!PropertyHandle)
		{
			return;
		}

		check(PropertyHandle->HasMetaData(TEXT("InteractorName")));
		FDetailWidgetRow::FCustomMenuData CustomMenuData(
			FUIAction(
				FExecuteAction::CreateLambda([PropertyHandle]()
					{
						const FString& InteractorName = PropertyHandle->GetMetaData(TEXT("InteractorName"));
						FPlatformApplicationMisc::ClipboardCopy(*InteractorName);
					})),
			LOCTEXT("CopyInteractorName", "Copy Interactor Name"),
			LOCTEXT("CopyInteractorNameToolTip", "Copy name that can be used to set this property via the Cloth Asset Interactor"),
			FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy")
		);
		InWidgetRow.CustomMenuItems.Add(CustomMenuData);
	}

} // namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
