// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNodeDetailExtension.h"

namespace UE::Chaos::ClothAsset
{
	struct FClothSimulationNodeDetailExtender : public UE::Dataflow::IDataflowNodeDetailExtension
	{
		FClothSimulationNodeDetailExtender() = default;
		static const FName Name;

		// IDataflowNodeDetailExtension
		virtual FName GetName() const override;

		// IDetailPropertyExtensionHandler
		virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;
		virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	};
} // namespace UE::Chaos::ClothAsset
