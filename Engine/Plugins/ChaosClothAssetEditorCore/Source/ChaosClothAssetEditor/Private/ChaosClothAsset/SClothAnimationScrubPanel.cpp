// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SClothAnimationScrubPanel.h"
#include "Widgets/SBoxPanel.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "SScrubControlPanel.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "ClothAnimationScrubPanel"

void SClothAnimationScrubPanel::Construct( const SClothAnimationScrubPanel::FArguments& InArgs, const TWeakPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> InPreviewScene)
{
	PreviewSceneWeakPtr = InPreviewScene;

	// Skip adding the the Loop button so we can add our own
	TArray<FTransportControlWidget> TransportControlWidgets;
	for (const ETransportControlWidgetType Type : TEnumRange<ETransportControlWidgetType>())
	{
		if ((Type != ETransportControlWidgetType::Custom) && (Type != ETransportControlWidgetType::Loop))
		{
			TransportControlWidgets.Add(FTransportControlWidget(Type));
		}
	}
	const FTransportControlWidget NewWidget(FOnMakeTransportWidget::CreateSP(this, &SClothAnimationScrubPanel::OnCreatePreviewPlaybackModeWidget));
	TransportControlWidgets.Add(NewWidget);

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		.AddMetaData<FTagMetaData>(TEXT("ClothAnimScrub.Scrub"))
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill) 
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.Padding(0.0f)
		[
			SAssignNew(ScrubControlPanel, SScrubControlPanel)
			.IsEnabled(true)
			.Value(this, &SClothAnimationScrubPanel::GetScrubValue)
			.NumOfKeys(this, &SClothAnimationScrubPanel::GetNumberOfKeys)
			.SequenceLength(this, &SClothAnimationScrubPanel::GetSequenceLength)
			.DisplayDrag(this, &SClothAnimationScrubPanel::GetDisplayDrag)
			.OnValueChanged(this, &SClothAnimationScrubPanel::OnValueChanged)
			.OnBeginSliderMovement(this, &SClothAnimationScrubPanel::OnBeginSliderMovement)
			.OnClickedForwardPlay(this, &SClothAnimationScrubPanel::OnClick_Forward)
			.OnClickedForwardStep(this, &SClothAnimationScrubPanel::OnClick_Forward_Step)
			.OnClickedForwardEnd(this, &SClothAnimationScrubPanel::OnClick_Forward_End)
			.OnClickedBackwardPlay(this, &SClothAnimationScrubPanel::OnClick_Backward)
			.OnClickedBackwardStep(this, &SClothAnimationScrubPanel::OnClick_Backward_Step)
			.OnClickedBackwardEnd(this, &SClothAnimationScrubPanel::OnClick_Backward_End)
			.OnTickPlayback(this, &SClothAnimationScrubPanel::OnTickPlayback)
			.OnGetPlaybackMode(this, &SClothAnimationScrubPanel::GetPlaybackMode)
			.ViewInputMin(InArgs._ViewInputMin)
			.ViewInputMax(InArgs._ViewInputMax)
			.bDisplayAnimScrubBarEditing(false)
			.bAllowZoom(false)
			.IsRealtimeStreamingMode(false)
			.TransportControlWidgetsToCreate(TransportControlWidgets)
		]
	];


	if (const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PinnedPreviewScene = PreviewSceneWeakPtr.Pin())
	{
		if (UChaosClothPreviewSceneDescription* const SceneDescription = PinnedPreviewScene->GetPreviewSceneDescription())
		{
			SceneDescription->ClothPreviewSceneDescriptionChanged.AddSP(this, &SClothAnimationScrubPanel::ApplyPlaybackSettings);
		}
	}

	ApplyPlaybackSettings();
}


TSharedRef<SWidget> SClothAnimationScrubPanel::OnCreatePreviewPlaybackModeWidget()
{
	PreviewPlaybackModeButton = SNew(SButton)
		.OnClicked(this, &SClothAnimationScrubPanel::OnClick_PreviewPlaybackMode)
		.ButtonStyle( FAppStyle::Get(), "Animation.PlayControlsButton" )
		.IsFocusable(false)
		.ToolTipText_Lambda([&]()
		{
			if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Default)
			{
				return LOCTEXT("PlaybackModeDefaultTooltip", "Linear playback");
			}
			else if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Looping)
			{
				return LOCTEXT("PlaybackModeLoopingTooltip", "Looping playback");
			}
			else
			{
				return LOCTEXT("PlaybackModePingPongTooltip", "Ping pong playback");
			}
		})
		.ContentPadding(0.0f);

	TWeakPtr<SButton> WeakButton = PreviewPlaybackModeButton;

	PreviewPlaybackModeButton->SetContent(SNew(SImage)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Image_Lambda([&, WeakButton]()
		{
			if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Default)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Disabled");
			}
			else if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Looping)
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.Enabled");
			}
			else
			{
				return FAppStyle::Get().GetBrush("Animation.Loop.SelectionRange");		// TODO: Replace with a back and forth type icon
			}
		})
	);

	TSharedRef<SHorizontalBox> PreviewPlaybackModeBox = SNew(SHorizontalBox);
	PreviewPlaybackModeBox->AddSlot()
	.AutoWidth()
	[
		PreviewPlaybackModeButton.ToSharedRef()
	];

	return PreviewPlaybackModeBox;
}

FReply SClothAnimationScrubPanel::OnClick_Forward_Step()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->StepForward();
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_Forward_End()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->SetPosition(PreviewInstance->GetLength(), false);
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_Backward_Step()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->StepBackward();
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_Backward_End()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->SetPosition(0.f, false);
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_Forward()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		const bool bIsReverse = PreviewInstance->IsReverse();
		const bool bIsPlaying = PreviewInstance->IsPlaying();

		// if current bIsReverse and bIsPlaying, we'd like to just turn off reverse
		if (bIsReverse && bIsPlaying)
		{
			PreviewInstance->SetReverse(false);
		}
		// already playing, simply pause
		else if (bIsPlaying) 
		{
			PreviewInstance->SetPlaying(false);
		}
		// if not playing, play forward
		else 
		{
			//if we're at the end of the animation, jump back to the beginning before playing
			if ( GetScrubValue() >= GetSequenceLength() )
			{
				PreviewInstance->SetPosition(0.0f, false);
			}

			PreviewInstance->SetReverse(false);
			PreviewInstance->SetPlaying(true);
		}
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_Backward()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		const bool bIsReverse = PreviewInstance->IsReverse();
		const bool bIsPlaying = PreviewInstance->IsPlaying();

		// if currently playing forward, just simply turn on reverse
		if (!bIsReverse && bIsPlaying)
		{
			PreviewInstance->SetReverse(true);
		}
		else if (bIsPlaying)
		{
			PreviewInstance->SetPlaying(false);
		}
		else
		{
			// if we're at the beginning of the animation, jump back to the end before playing
			if ( GetScrubValue() <= 0.0f )
			{
				PreviewInstance->SetPosition(GetSequenceLength(), false);
			}

			PreviewInstance->SetPlaying(true);
			PreviewInstance->SetReverse(true);
		}
	}

	return FReply::Handled();
}

FReply SClothAnimationScrubPanel::OnClick_PreviewPlaybackMode()
{
	if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Default)
	{
		PreviewPlaybackMode = EClothPreviewPlaybackMode::Looping;
		
		if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
		{
			if (PreviewInstance->CurrentAsset)
			{
				// If we paused due to hitting the end point, start playing again when entering loop mode
				const float CurrentTime = PreviewInstance->GetCurrentTime();
				const float PlayRate = PreviewInstance->GetPlayRate();
				const float AssetPlayLength = PreviewInstance->CurrentAsset->GetPlayLength();
				if (PlayRate < 0.0 && CurrentTime <= 0.0)
				{
					PreviewInstance->SetPlaying(true);
				}
				else if (PlayRate > 0.0 && CurrentTime >= AssetPlayLength)
				{
					PreviewInstance->SetPlaying(true);
				}
			}
		}
	}
	else if (PreviewPlaybackMode == EClothPreviewPlaybackMode::Looping)
	{
		PreviewPlaybackMode = EClothPreviewPlaybackMode::PingPong;
	}
	else
	{
		PreviewPlaybackMode = EClothPreviewPlaybackMode::Default;

		if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
		{
			// If we're switching to linear playback, set it to forward mode
			PreviewInstance->SetReverse(false);
		}
	}

	ApplyPlaybackSettings();

	return FReply::Handled();
}

void SClothAnimationScrubPanel::ApplyPlaybackSettings()
{
	UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance();

	switch (PreviewPlaybackMode)
	{
	case EClothPreviewPlaybackMode::Default:
		if (PreviewInstance)
		{
			PreviewInstance->SetLooping(false);
		}
		break;
	case EClothPreviewPlaybackMode::Looping:
		if (PreviewInstance)
		{
			PreviewInstance->SetLooping(true);
		}
		break;
	case EClothPreviewPlaybackMode::PingPong:
		if (PreviewInstance)
		{
			PreviewInstance->SetLooping(false);
		}
		break;
	}
}

void SClothAnimationScrubPanel::OnTickPlayback(double InCurrentTime, float InDeltaTime)
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		if (PreviewInstance->CurrentAsset)
		{
			const float CurrentTime = PreviewInstance->GetCurrentTime();
			const float PlayRate = PreviewInstance->GetPlayRate();
			const float AssetPlayLength = PreviewInstance->CurrentAsset->GetPlayLength();

			if (PreviewPlaybackMode == EClothPreviewPlaybackMode::PingPong)
			{
				if (PlayRate < 0.0 && CurrentTime <= 0.0)
				{
					PreviewInstance->SetReverse(!PreviewInstance->IsReverse());
				}
				else if (PlayRate > 0.0 && CurrentTime >= AssetPlayLength)
				{
					PreviewInstance->SetReverse(!PreviewInstance->IsReverse());
				}
			}
		}
	}
}

EPlaybackMode::Type SClothAnimationScrubPanel::GetPlaybackMode() const
{
	if (const UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		if (PreviewInstance->IsPlaying())
		{
			return PreviewInstance->IsReverse() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
		}
		return EPlaybackMode::Stopped;
	}
	
	return EPlaybackMode::Stopped;
}

void SClothAnimationScrubPanel::OnValueChanged(float NewValue)
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPosition(NewValue);
	}
}

void SClothAnimationScrubPanel::OnBeginSliderMovement()
{
	if (UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		PreviewInstance->SetPlaying(false);
	}
}

uint32 SClothAnimationScrubPanel::GetNumberOfKeys() const
{
	if (const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewScene = PreviewSceneWeakPtr.Pin())
	{
		if (UAnimSingleNodeInstance* const PreviewInstance = PreviewScene->GetPreviewAnimInstance())	// non-const because UAnimSingleNodeInstance::GetLength() is non-const
		{
			const float Length = PreviewInstance->GetLength();

			// if anim sequence, use correct num frames
			int32 NumKeys = (int32)(Length / 0.0333f);

			if (PreviewInstance->CurrentAsset)
			{
				if (PreviewInstance->CurrentAsset->IsA(UAnimSequenceBase::StaticClass()))
				{
					NumKeys = CastChecked<UAnimSequenceBase>(PreviewInstance->CurrentAsset)->GetNumberOfSampledKeys();
				}
				else if (PreviewInstance->CurrentAsset->IsA(UBlendSpace::StaticClass()))
				{
					// Blendspaces dont display frame notches, so just return 0 here
					NumKeys = 0;
				}
			}
			return NumKeys;
		}
	}

	return 1;
}

float SClothAnimationScrubPanel::GetSequenceLength() const
{
	if (const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewScene = PreviewSceneWeakPtr.Pin())
	{
		if (UAnimSingleNodeInstance* const PreviewInstance = PreviewScene->GetPreviewAnimInstance())	// non-const because UAnimSingleNodeInstance::GetLength() is non-const
		{
			return PreviewInstance->GetLength();
		}
	}

	return 0.f;
}

float SClothAnimationScrubPanel::GetScrubValue() const
{
	if (const UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance())
	{
		return PreviewInstance->GetCurrentTime(); 
	}

	return 0.f;
}

bool SClothAnimationScrubPanel::GetDisplayDrag() const
{
	const UAnimSingleNodeInstance* const PreviewInstance = GetPreviewAnimationInstance();
	if (PreviewInstance && PreviewInstance->CurrentAsset)
	{
		return true;
	}

	return false;
}

UAnimSingleNodeInstance* SClothAnimationScrubPanel::GetPreviewAnimationInstance()
{
	if (const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewScene = PreviewSceneWeakPtr.Pin())
	{
		return PreviewScene->GetPreviewAnimInstance();
	}

	return nullptr;
}

const UAnimSingleNodeInstance* SClothAnimationScrubPanel::GetPreviewAnimationInstance() const
{
	if (const TSharedPtr<const UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewScene = PreviewSceneWeakPtr.Pin())
	{
		return PreviewScene->GetPreviewAnimInstance();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
