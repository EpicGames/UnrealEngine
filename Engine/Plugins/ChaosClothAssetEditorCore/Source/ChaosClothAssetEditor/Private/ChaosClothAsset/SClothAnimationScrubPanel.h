// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "ITransportControl.h"  // EPlaybackMode::Type

class SScrubControlPanel;
class SButton;
class UAnimSingleNodeInstance;
namespace UE::Chaos::ClothAsset
{
class FChaosClothPreviewScene;
}

///
/// A simple panel containing animation controls to operate on a UAnimSingleNodeInstance
///
class SClothAnimationScrubPanel : public SCompoundWidget
{
private:

	enum class EClothPreviewPlaybackMode : int32
	{
		Default,
		Looping,
		PingPong
	};
	
	SLATE_BEGIN_ARGS(SClothAnimationScrubPanel)	{}
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TWeakPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> InPreviewScene );

	TSharedRef<SWidget> OnCreatePreviewPlaybackModeWidget();

	// notifiers 
	FReply OnClick_Forward_Step();
	FReply OnClick_Forward_End();
	FReply OnClick_Backward_Step();
	FReply OnClick_Backward_End();
	FReply OnClick_Forward();
	FReply OnClick_Backward();
	FReply OnClick_PreviewPlaybackMode();

	void ApplyPlaybackSettings();

	void OnTickPlayback(double InCurrentTime, float InDeltaTime);

	void OnValueChanged(float NewValue);
	void OnBeginSliderMovement();

	EPlaybackMode::Type GetPlaybackMode() const;
	float GetScrubValue() const;

	UAnimSingleNodeInstance* GetPreviewAnimationInstance();
	const UAnimSingleNodeInstance* GetPreviewAnimationInstance() const;

	uint32 GetNumberOfKeys() const;
	float GetSequenceLength() const;

	bool GetDisplayDrag() const;

	TWeakPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewSceneWeakPtr;

	TSharedPtr<SScrubControlPanel> ScrubControlPanel;

	TSharedPtr<SButton> PreviewPlaybackModeButton;
	EClothPreviewPlaybackMode PreviewPlaybackMode = EClothPreviewPlaybackMode::Looping;
};
