// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SClothCollectionOutliner.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Templates/EnableIf.h"

#define LOCTEXT_NAMESPACE "ClothCollectionOutliner"

const FName FClothCollectionHeaderData::ColumnZeroName = FName("ID");

void SClothCollectionOutlinerRow::Construct(const FArguments& InArgs,
	TSharedRef<STableViewBase> OwnerTableView,
	const TSharedPtr<const FClothCollectionHeaderData>& InHeaderData,
	const TSharedPtr<const FClothCollectionItem>& InItemToEdit)
{
	HeaderData = InHeaderData;
	Item = InItemToEdit;

	SMultiColumnTableRow<TSharedPtr<const FClothCollectionItem>>::Construct(
		FSuperRowType::FArguments()
		.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
		, OwnerTableView);
}


TSharedRef<SWidget> SClothCollectionOutlinerRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	int32 FoundIndex;
	if (HeaderData->AttributeNames.Find(ColumnName, FoundIndex))
	{
		const FString& AttrValue = Item->AttributeValues[FoundIndex];

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(AttrValue))
			];
	}

	return SNullWidget::NullWidget;
}


void SClothCollectionOutliner::Construct(const FArguments& InArgs)
{
	ClothCollection = InArgs._ClothCollection;
	SelectedGroupName = InArgs._SelectedGroupName;

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible);

	if (ClothCollection.IsValid())
	{
		RegenerateHeader();
		RepopulateListView();
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SelectedGroupNameComboBox, SComboBox<FName>)
			.OptionsSource(&ClothCollectionGroupNames)
			.OnSelectionChanged(SComboBox<FName>::FOnSelectionChanged::CreateLambda(
				[this](FName SelectedName, ESelectInfo::Type)
				{
					SetSelectedGroupName(SelectedName);
				}))
			.OnGenerateWidget(SComboBox<FName>::FOnGenerateWidget::CreateLambda(
				[](FName Item)
				{
					return SNew(STextBlock)
						.Text(FText::FromName(Item));
				}))
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return FText::FromName(GetSelectedGroupName());
				})
			]
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.f))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(ListView, SListView<TSharedPtr<const FClothCollectionItem>>)
				.SelectionMode(ESelectionMode::None)
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SClothCollectionOutliner::GenerateRow)
				.HeaderRow(HeaderRowWidget)
			]
		]
	];
}

void SClothCollectionOutliner::SetClothCollection(TSharedPtr<FManagedArrayCollection> InClothCollection)
{
	if (!InClothCollection.IsValid() && ClothCollection.IsValid())
	{
		// Going from valid node selected to no node selected
		SavedLastValidGroupName = SelectedGroupName;
	}
	else if (InClothCollection.IsValid() && !ClothCollection.IsValid())
	{
		// Going from no node selected to a valid node selected
		SelectedGroupName = SavedLastValidGroupName;
	}

	ClothCollection = InClothCollection;

	if (ClothCollection.IsValid())
	{
		ClothCollectionGroupNames = ClothCollection->GroupNames();
		RegenerateHeader();
		RepopulateListView();
	}
	else
	{
		ClothCollectionGroupNames.Reset();
		HeaderRowWidget->ClearColumns();
		if (HeaderData)
		{
			HeaderData->AttributeNames.Empty();
		}
		ListItems.Empty();
		SelectedGroupName = FName();
	}
	// Refresh the combo box with the new set of group names after ClothCollectionGroupNames changed
	SelectedGroupNameComboBox->RefreshOptions();
}

void SClothCollectionOutliner::SetSelectedGroupName(const FName& InSelectedGroupName)
{
	SelectedGroupName = InSelectedGroupName;

	RegenerateHeader();
	RepopulateListView();
}

const FName& SClothCollectionOutliner::GetSelectedGroupName() const
{
	return SelectedGroupName;
}

void SClothCollectionOutliner::RegenerateHeader()
{
	constexpr float CustomFillWidth = 2.0f;

	if (!ClothCollection.IsValid())
	{
		return;
	}

	HeaderRowWidget->ClearColumns();

	HeaderData = MakeShared<FClothCollectionHeaderData>();
	HeaderData->AttributeNames.Add(FClothCollectionHeaderData::ColumnZeroName);
	HeaderData->AttributeNames.Append(ClothCollection->AttributeNames(SelectedGroupName));

	int32 UnnamedCount = 0;
	for (int32 AttributeNameIndex = 0; AttributeNameIndex < HeaderData->AttributeNames.Num(); ++AttributeNameIndex)
	{
		FName AttrName = HeaderData->AttributeNames[AttributeNameIndex];
		if (AttrName == NAME_None)  // AddColumn needs a name, otherwise it crashes
		{
			AttrName = FName(FString::Format(TEXT("Unnamed{0}"), { UnnamedCount++ }));
		}

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(AttrName)
			.DefaultLabel(FText::FromName(AttrName))
			.FillWidth(CustomFillWidth)
		);
	}
}

namespace ClothCollectionOutlinerHelpers
{
	template<typename T>
	FString AttributeValueToString(const T& Value)
	{
		return Value.ToString();
	}

	FString AttributeValueToString(float Value)
	{
		return FString::SanitizeFloat(Value);
	}

	FString AttributeValueToString(int32 Value)
	{
		return FString::FromInt(Value);
	}

	FString AttributeValueToString(uint8 Value)
	{
		return FString::FromInt(Value);
	}

	FString AttributeValueToString(const FString& Value)
	{
		return Value;
	}

	FString AttributeValueToString(const FIntVector2& Value)
	{
		return FString::Printf(TEXT("X=%d Y=%d"), Value.X, Value.Y);
	}

	template<typename T>
	FString AttributeValueToString(const TArray<T>& Array)
	{
		FString Out;
		for ( int32 ArrayIndex = 0; ArrayIndex < Array.Num(); ++ArrayIndex )
		{
			Out += AttributeValueToString(Array[ArrayIndex]);

			if (ArrayIndex != Array.Num() - 1)
			{
				Out += "; ";
			}
		}
		return Out;
	}

	template<typename T>
	FString AttributeValueToString(const TSet<T>& Set)
	{
		FString Out;
		typename TSet<T>::TConstIterator Iter = Set.CreateConstIterator();
		while(Iter)
		{
			Out += AttributeValueToString(*Iter);

			if (++Iter)
			{
				Out += "; ";
			}
		}
		return Out;
	}

	template<typename T>
	FString AttributeValueToString(const FManagedArrayCollection& ClothCollection, const FName& AttributeName, const FName& GroupName, int32 AttributeArrayIndex)
	{
		const TManagedArray<T>* const Array = ClothCollection.FindAttributeTyped<T>(AttributeName, GroupName);
		if (Array == nullptr)
		{
			return FString("(Unknown Attribute)");
		}

		return AttributeValueToString((*Array)[AttributeArrayIndex]);
	}

	FString AttributeValueToString(const FManagedArrayCollection& ClothCollection, const FName& AttributeName, const FName& GroupName, int32 AttributeArrayIndex)
	{
		const FManagedArrayCollection::EArrayType ArrayType = ClothCollection.GetAttributeType(AttributeName, GroupName);

		FString ValueAsString;

		switch (ArrayType)
		{
		case FManagedArrayCollection::EArrayType::FVectorType:
			ValueAsString = AttributeValueToString<FVector3f>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DType:
			ValueAsString = AttributeValueToString<FVector2f>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FFloatType:
			ValueAsString = AttributeValueToString<float>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FIntVectorType:
			ValueAsString = AttributeValueToString<FIntVector3>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FVector2DArrayType:
			ValueAsString = AttributeValueToString<TArray<FVector2f>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FLinearColorType:
			ValueAsString = AttributeValueToString<FLinearColor>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FInt32Type:
			ValueAsString = AttributeValueToString<int32>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FInt32ArrayType:
			ValueAsString = AttributeValueToString<TArray<int32>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FFloatArrayType:
			ValueAsString = AttributeValueToString<TArray<float>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FStringType:
			ValueAsString = AttributeValueToString<FString>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2Type:
			ValueAsString = AttributeValueToString<FIntVector2>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector2ArrayType:
			ValueAsString = AttributeValueToString<TArray<FIntVector2>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FIntArrayType:
			ValueAsString = AttributeValueToString<TSet<int32>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FUInt8Type:
			ValueAsString = AttributeValueToString<uint8>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FIntVector3ArrayType:
			ValueAsString = AttributeValueToString<TArray<FIntVector3>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		case FManagedArrayCollection::EArrayType::FVector4fArrayType:
			ValueAsString = AttributeValueToString<TArray<FVector4f>>(ClothCollection, AttributeName, GroupName, AttributeArrayIndex);
			break;
		default:
			ensure(false);
			ValueAsString = "(Unknown Data Type)";
		}

		return ValueAsString;
	}
}

void SClothCollectionOutliner::RepopulateListView()
{
	ListItems.Empty();

	if (!ClothCollection.IsValid())
	{
		return;
	}

	const int32 NumElements = ClothCollection->NumElements(SelectedGroupName);

	for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
	{
		const TSharedPtr<FClothCollectionItem> NewItem = MakeShared<FClothCollectionItem>();
		NewItem->AttributeValues.SetNum(HeaderData->AttributeNames.Num());

		for (int32 AttributeNameIndex = 0; AttributeNameIndex < HeaderData->AttributeNames.Num(); ++AttributeNameIndex)
		{
			const FName& AttributeName = HeaderData->AttributeNames[AttributeNameIndex];
			if (AttributeName == FClothCollectionHeaderData::ColumnZeroName)
			{
				NewItem->AttributeValues[AttributeNameIndex] = FString::FromInt(ElementIndex);
			}
			else
			{
				NewItem->AttributeValues[AttributeNameIndex] = ClothCollectionOutlinerHelpers::AttributeValueToString(*ClothCollection, AttributeName, SelectedGroupName, ElementIndex);
			}
		}

		ListItems.Add(NewItem);
	}

	ListView->RequestListRefresh();
}

TSharedRef<ITableRow> SClothCollectionOutliner::GenerateRow(TSharedPtr<const FClothCollectionItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SClothCollectionOutlinerRow, OwnerTable, this->HeaderData, InItem);
}


#undef LOCTEXT_NAMESPACE
