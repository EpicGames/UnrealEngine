// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"

struct FManagedArrayCollection;

struct FClothCollectionHeaderData
{
	TArray<FName> AttributeNames;

	// first column is used to display the numeric index of each element
	static const FName ColumnZeroName;	
};

struct FClothCollectionItem
{
	TArray<FString> AttributeValues;
};

class SClothCollectionOutlinerRow : public SMultiColumnTableRow<TSharedPtr<const FClothCollectionItem>>
{
public:
	SLATE_BEGIN_ARGS(SClothCollectionOutlinerRow) {}
		SLATE_ARGUMENT(TSharedPtr<const FClothCollectionHeaderData>, HeaderData)
		SLATE_ARGUMENT(TSharedPtr<const FClothCollectionItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, const TSharedPtr<const FClothCollectionHeaderData>& InHeaderData, const TSharedPtr<const FClothCollectionItem>& InItemToEdit);

	// SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	TSharedPtr<const FClothCollectionHeaderData> HeaderData;
	TSharedPtr<const FClothCollectionItem> Item;
};

/// Data visualizer for FClothCollection. Pulls out all Attribute names and values from a given Cloth Collection with a given Group name and adds them to a table
class SClothCollectionOutliner : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SClothCollectionOutliner) {}
		SLATE_ARGUMENT(TSharedPtr<FManagedArrayCollection>, ClothCollection)
		SLATE_ARGUMENT(FName, SelectedGroupName)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetClothCollection(TSharedPtr<FManagedArrayCollection> ClothCollection);

	///  Changing the selected group name will also automatically rebuild the header and all rows
	void SetSelectedGroupName(const FName& SelectedGroupName);

	const FName& GetSelectedGroupName() const;

	TSharedRef<ITableRow> GenerateRow(TSharedPtr<const FClothCollectionItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:

	TSharedPtr<FManagedArrayCollection> ClothCollection;

	TSharedPtr<SComboBox<FName>> SelectedGroupNameComboBox;
	TArray<FName> ClothCollectionGroupNames;		// Data source for SelectedGroupNameComboBox

	FName SelectedGroupName;
	FName SavedLastValidGroupName;

	TSharedPtr<SListView<TSharedPtr<const FClothCollectionItem>>> ListView;
	TArray<TSharedPtr<const FClothCollectionItem>> ListItems;

	TSharedPtr<FClothCollectionHeaderData> HeaderData;
	TSharedPtr<SHeaderRow> HeaderRowWidget;

	void RegenerateHeader();
	void RepopulateListView();

};
