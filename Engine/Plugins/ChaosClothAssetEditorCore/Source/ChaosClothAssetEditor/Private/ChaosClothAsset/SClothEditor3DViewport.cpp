// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SClothEditor3DViewport.h"

#include "AdvancedPreviewSceneMenus.h"
#include "ClothEditorSimulationVisualization.h"
#include "ClothEditorViewportToolbarSections.h"
#include "ChaosClothAsset/ClothEditor3DViewportClient.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "ChaosClothAsset/SClothAnimationScrubPanel.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "EditorModeTools.h"
#include "ToolMenus.h"
#include "PreviewProfileController.h"
#include "Widgets/Layout/SBorder.h"
#include "Components/SkeletalMeshComponent.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SChaosClothAssetEditor3DViewport"

void SChaosClothAssetEditor3DViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	SAssetEditorViewport::FArguments ParentArgs;
	ParentArgs._EditorViewportClient = InArgs._EditorViewportClient;
	if (InArgs._ViewportSize.IsSet())
	{
		ParentArgs._ViewportSize = InArgs._ViewportSize;
	}
	ToolkitCommandList = InArgs._ToolkitCommandList;
	SAssetEditorViewport::Construct(ParentArgs, InViewportConstructionArgs);
	Client->VisibilityDelegate.BindSP(this, &SChaosClothAssetEditor3DViewport::IsVisible);

	ViewportOverlay->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		.FillWidth(1)
		.Padding(10.0f, 0.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
			.Visibility_Raw(this, &SChaosClothAssetEditor3DViewport::GetAnimControlVisibility)
			.Padding(10.0f, 2.0f)
			[
				SNew(SClothAnimationScrubPanel, GetPreviewScene())
				.ViewInputMin(this, &SChaosClothAssetEditor3DViewport::GetViewMinInput)
				.ViewInputMax(this, &SChaosClothAssetEditor3DViewport::GetViewMaxInput)
			]
		]
	];
	
	if (TSharedPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> PreviewScene = GetPreviewScene().Pin())
	{
		UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(PreviewScene, Client);
	}
}

TWeakPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> SChaosClothAssetEditor3DViewport::GetPreviewScene()
{
	const TSharedPtr<UE::Chaos::ClothAsset::FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<UE::Chaos::ClothAsset::FChaosClothAssetEditor3DViewportClient>(Client);
	return ClothViewportClient->GetClothPreviewScene();
}

TWeakPtr<const UE::Chaos::ClothAsset::FChaosClothPreviewScene> SChaosClothAssetEditor3DViewport::GetPreviewScene() const
{
	const TSharedPtr<const UE::Chaos::ClothAsset::FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<UE::Chaos::ClothAsset::FChaosClothAssetEditor3DViewportClient>(Client);
	return ClothViewportClient->GetClothPreviewScene();
}

TSharedPtr<IPreviewProfileController> SChaosClothAssetEditor3DViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();	
}

void SChaosClothAssetEditor3DViewport::BindCommands()
{
	using namespace UE::Chaos::ClothAsset;

	SAssetEditorViewport::BindCommands();
	const FChaosClothAssetEditorCommands& CommandInfos = FChaosClothAssetEditorCommands::Get();

	ToolkitCommandList->MapAction(
		CommandInfos.TogglePreviewWireframe,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			ClothViewportClient->EnableRenderMeshWireframe(!ClothViewportClient->RenderMeshWireframeEnabled());
		}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client)->RenderMeshWireframeEnabled(); }));

	ToolkitCommandList->MapAction(
		CommandInfos.SoftResetSimulation,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			ClothViewportClient->SoftResetSimulation();
		}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return false; }));

	ToolkitCommandList->MapAction(
		CommandInfos.HardResetSimulation,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			ClothViewportClient->HardResetSimulation();
		}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return false; }));


	ToolkitCommandList->MapAction(
		CommandInfos.ToggleSimulationSuspended,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);

			const bool bIsSuspended = ClothViewportClient->IsSimulationSuspended();
			if (bIsSuspended)
			{
				ClothViewportClient->ResumeSimulation();
			}
			else
			{
				ClothViewportClient->SuspendSimulation();
			}
		}),
		FCanExecuteAction::CreateLambda([this]() { return true; }),
		FIsActionChecked::CreateLambda([this]() { return StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client)->IsSimulationSuspended(); }) );

	ToolkitCommandList->MapAction(
		CommandInfos.LODAuto,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			ClothViewportClient->SetLODLevel(INDEX_NONE);
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			return ClothViewportClient->IsLODSelected(INDEX_NONE);
		}
		));

	ToolkitCommandList->MapAction(
		CommandInfos.LOD0,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			ClothViewportClient->SetLODLevel(0);
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
			return ClothViewportClient->IsLODSelected(0);
		}
		));

	// all other LODs will be added dynamically

}

TSharedPtr<SWidget> SChaosClothAssetEditor3DViewport::BuildViewportToolbar()
{
	using namespace UE::Chaos::ClothAsset;
	
	const FName ViewportToolbarName = "ChaosClothEditor.3DViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolbarMenu->StyleName = "ViewportToolbar";

		ToolbarMenu->AddSection("Left");

		{
			FToolMenuSection& RightSection = ToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));

			{
				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
				UToolMenu* ViewModesMenu = UToolMenus::Get()->ExtendMenu(UToolMenus::JoinMenuPaths(ViewportToolbarName, "ViewModes"));
				FToolMenuSection& ViewSection = ViewModesMenu->FindOrAddSection("Cloth", LOCTEXT("ClothViewModeSection", "Cloth"));

				FToolMenuEntry& WireframeEntry = ViewSection.AddMenuEntry(FChaosClothAssetEditorCommands::Get().TogglePreviewWireframe);
				WireframeEntry.SetShowInToolbarTopLevel(true);
				WireframeEntry.ToolBarData.ResizeParams.ClippingPriority = 2000;
			}

			{
				// LOD
				RightSection.AddDynamicEntry("DynamicLOD", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
					{
						if (UUnrealEdViewportToolbarContext* Context = Section.FindContext<UUnrealEdViewportToolbarContext>())
						{
							if (TSharedPtr<SEditorViewport> EditorViewport = Context->Viewport.Pin())
							{
								if (TSharedPtr<FEditorViewportClient> Client = EditorViewport->GetViewportClient())
								{
									TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
									Section.AddEntry(UE::UnrealEd::CreatePreviewLODSelectionSubmenu(ClothClient));
								}
							}
						}
					}));
			}

			{
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
				UToolMenu* AssetViewerMenu = UToolMenus::Get()->ExtendMenu(UToolMenus::JoinMenuPaths(ViewportToolbarName, "AssetViewerProfile"));
				FToolMenuSection& SimulationSection = AssetViewerMenu->FindOrAddSection("SimulationControls", LOCTEXT("SimulationControlsSection", "Simulation Playback Controls"));

				// Simulation Controls
				FToolMenuEntryToolBarData ToolBarData;
				ToolBarData.BlockGroupName = "SimulationControlGroup";
				ToolBarData.LabelOverride = FText::GetEmpty();
				ToolBarData.ResizeParams.ClippingPriority = 2000;

				FToolMenuEntry& HardReset = SimulationSection.AddMenuEntry(FChaosClothAssetEditorCommands::Get().HardResetSimulation);
				HardReset.ToolBarData = ToolBarData;
				HardReset.SetShowInToolbarTopLevel(true);

				FToolMenuEntry& SoftReset = SimulationSection.AddMenuEntry(FChaosClothAssetEditorCommands::Get().SoftResetSimulation);
				SoftReset.ToolBarData = ToolBarData;
				SoftReset.SetShowInToolbarTopLevel(true);

				FToolMenuEntry& ToggleSimulation = SimulationSection.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleSimulationSuspended);
				ToggleSimulation.ToolBarData = ToolBarData;
				ToggleSimulation.SetShowInToolbarTopLevel(true);
			}
		}
	}

	FToolMenuContext Context;
	{
		Context.AppendCommandList(GetCommandList());
		Context.AppendCommandList(ToolkitCommandList);
		Context.AddExtender(GetExtenders());

		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		Context.AddObject(ContextObject);
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, Context);
}

void SChaosClothAssetEditor3DViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
	.Padding(FMargin(4.0f, 3.0f, 0.0f, 0.0f))
	[
		SNew(SBox)
		[
			SNew(SRichTextBlock)
			.DecoratorStyleSet(&FAppStyle::Get())
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimViewport.MessageText"))
			.Text(this, &SChaosClothAssetEditor3DViewport::GetViewportDisplayString)
		]
	];
}

bool SChaosClothAssetEditor3DViewport::IsVisible() const
{
	// Intentionally not calling SEditorViewport::IsVisible because it will return false if our simulation is more than 250ms.
	return ViewportWidget.IsValid();
}

void SChaosClothAssetEditor3DViewport::OnFocusViewportToSelection()
{
	using namespace UE::Chaos::ClothAsset;

	const TSharedPtr<const FChaosClothAssetEditor3DViewportClient> ViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
	const FBox PreviewBoundingBox = ViewportClient->PreviewBoundingBox();

	if (PreviewBoundingBox.IsValid && !(PreviewBoundingBox.Min == FVector::Zero() && PreviewBoundingBox.Max == FVector::Zero()))
	{
		Client->FocusViewportOnBox(PreviewBoundingBox);
	}
}

TSharedRef<class SEditorViewport> SChaosClothAssetEditor3DViewport::GetViewportWidget()
{
	return SharedThis(this);
}

FText SChaosClothAssetEditor3DViewport::GetViewportDisplayString() const
{
	using namespace UE::Chaos::ClothAsset;
	TSharedRef<FChaosClothAssetEditor3DViewportClient> ViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(GetViewportClient()).ToSharedRef();
	if (FClothEditorSimulationVisualization* const Visualization = ViewportClient->GetSimulationVisualization().Pin().Get())
	{
		return Visualization->GetDisplayString(ViewportClient->GetPreviewClothComponent());
	}
	return FText();
}

TSharedPtr<FExtender> SChaosClothAssetEditor3DViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

float SChaosClothAssetEditor3DViewport::GetViewMinInput() const
{
	return 0.0f;
}

float SChaosClothAssetEditor3DViewport::GetViewMaxInput() const
{
	using namespace UE::Chaos::ClothAsset;

	// (these are non-const because UAnimSingleNodeInstance::GetLength() is non-const)
	const TSharedPtr<FChaosClothAssetEditor3DViewportClient> ClothViewportClient = StaticCastSharedPtr<FChaosClothAssetEditor3DViewportClient>(Client);
	const TSharedPtr<FChaosClothPreviewScene> Scene = ClothViewportClient->GetClothPreviewScene().Pin();
	if (Scene)
	{
		if (UAnimSingleNodeInstance* const PreviewInstance = Scene->GetPreviewAnimInstance())
		{
			return PreviewInstance->GetLength();
		}
	}

	return 0.0f;
}

EVisibility SChaosClothAssetEditor3DViewport::GetAnimControlVisibility() const
{
	using namespace UE::Chaos::ClothAsset;

	const TSharedPtr<const FChaosClothPreviewScene> Scene = GetPreviewScene().Pin();
	return (Scene && Scene->GetSkeletalMeshComponent() && Scene->GetSkeletalMeshComponent()->GetSkeletalMeshAsset() && Scene->GetPreviewAnimInstance()) ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
