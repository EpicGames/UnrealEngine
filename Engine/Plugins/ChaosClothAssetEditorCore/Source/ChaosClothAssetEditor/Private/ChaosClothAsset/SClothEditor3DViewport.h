// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SAssetEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SBaseCharacterFXEditorViewport.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

namespace UE::Chaos::ClothAsset
{
class FChaosClothPreviewScene;
}
class SClothAnimationScrubPanel;

/**
 * Viewport used for 3D preview in cloth editor. Has a custom toolbar overlay at the top.
 */
class SChaosClothAssetEditor3DViewport : public SBaseCharacterFXEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{

public:

	SLATE_BEGIN_ARGS(SChaosClothAssetEditor3DViewport) {}
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);
		SLATE_ARGUMENT(TSharedPtr<FEditorViewportClient>, EditorViewportClient)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, ToolkitCommandList)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);

	// SAssetEditorViewport
	UE_API virtual void BindCommands() override;
	UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	UE_API virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	UE_API virtual bool IsVisible() const override;

	UE_API virtual void OnFocusViewportToSelection() override;

	// ICommonEditorViewportToolbarInfoProvider
	UE_API virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	UE_API virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override {}

private:

	// Use this command list if we want to enable editor-wide command chords/hotkeys. 
	// Use SEditorViewport::CommandList if we want command hotkeys to only be active when the mouse is in this viewport.
	TSharedPtr<FUICommandList> ToolkitCommandList;

	UE_API TWeakPtr<UE::Chaos::ClothAsset::FChaosClothPreviewScene> GetPreviewScene();
	UE_API TWeakPtr<const UE::Chaos::ClothAsset::FChaosClothPreviewScene> GetPreviewScene() const;

	UE_API virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	
	UE_API FText GetViewportDisplayString() const;

	UE_API float GetViewMinInput() const;
	UE_API float GetViewMaxInput() const;
	UE_API EVisibility GetAnimControlVisibility() const;
		 

};

#undef UE_API
