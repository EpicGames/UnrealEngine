// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClothEditorAdvancedPreviewDetailsTab.h"
#include "ChaosClothAsset/ClothEditorPreviewScene.h"
#include "IDetailsView.h"

SChaosClothEditorAdvancedPreviewDetailsTab::SChaosClothEditorAdvancedPreviewDetailsTab()
	: SAdvancedPreviewDetailsTab()
{
	PropertyChangedDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddLambda([this](UObject* Object, struct FPropertyChangedEvent& Event)
	{
		if (Object->IsA<UChaosClothPreviewSceneDescription>())
		{
			SettingsView->InvalidateCachedState();
		}
	});
}

SChaosClothEditorAdvancedPreviewDetailsTab::~SChaosClothEditorAdvancedPreviewDetailsTab()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PropertyChangedDelegateHandle);
}
