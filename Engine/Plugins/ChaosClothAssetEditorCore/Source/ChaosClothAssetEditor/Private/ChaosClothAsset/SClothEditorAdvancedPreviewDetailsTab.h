// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAdvancedPreviewDetailsTab.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

// Custom subclass of SAdvancedPreviewDetailsTab that allows invalidating the cached state of the SettingsView when the preview scene changes
class SChaosClothEditorAdvancedPreviewDetailsTab : public SAdvancedPreviewDetailsTab
{
public:

	UE_API SChaosClothEditorAdvancedPreviewDetailsTab();
	UE_API virtual ~SChaosClothEditorAdvancedPreviewDetailsTab() override;

private:

	FDelegateHandle PropertyChangedDelegateHandle;
};

#undef UE_API
