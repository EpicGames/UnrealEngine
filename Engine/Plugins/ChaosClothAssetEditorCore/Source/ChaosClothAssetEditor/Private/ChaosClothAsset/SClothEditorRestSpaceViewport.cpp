// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SClothEditorRestSpaceViewport.h"

#include "ClothEditorViewportToolbarSections.h"
#include "ChaosClothAsset/ClothEditorRestSpaceViewportClient.h"
#include "ChaosClothAsset/ClothEditorCommands.h"
#include "ChaosClothAsset/ClothEditorMode.h"
#include "EditorModeManager.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "SChaosClothAssetEditorRestSpaceViewport"

void SChaosClothAssetEditorRestSpaceViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs)
{
	RestSpaceViewportClient = InArgs._RestSpaceViewportClient;

	SAssetEditorViewport::FArguments ParentArgs;
	ParentArgs._EditorViewportClient = InArgs._RestSpaceViewportClient;
	if (InArgs._ViewportSize.IsSet())
	{
		ParentArgs._ViewportSize = InArgs._ViewportSize;
	}
	SAssetEditorViewport::Construct(ParentArgs, InViewportConstructionArgs);
	Client->VisibilityDelegate.BindSP(this, &SChaosClothAssetEditorRestSpaceViewport::IsVisible);
}

UChaosClothAssetEditorMode* SChaosClothAssetEditorRestSpaceViewport::GetEdMode() const
{
	if (const FEditorModeTools* const EditorModeTools = Client->GetModeTools())
	{
		if (UChaosClothAssetEditorMode* const ClothEdMode = Cast<UChaosClothAssetEditorMode>(EditorModeTools->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId)))
		{
			return ClothEdMode;
		}
	}
	return nullptr;
}

void SChaosClothAssetEditorRestSpaceViewport::BindCommands()
{
	using namespace UE::Chaos::ClothAsset;

	SAssetEditorViewport::BindCommands();

	const FChaosClothAssetEditorCommands& CommandInfos = FChaosClothAssetEditorCommands::Get();

	CommandList->MapAction(
		CommandInfos.SetConstructionMode2D,
		FExecuteAction::CreateLambda([this]()
		{
			if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				ClothEdMode->SetConstructionViewMode(EClothPatternVertexType::Sim2D);
			}
		}),
		FCanExecuteAction::CreateLambda([this]() 
		{ 
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanChangeConstructionViewModeTo(EClothPatternVertexType::Sim2D);
			}
			return false; 
		}),
		FIsActionChecked::CreateLambda([this]() 
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->GetConstructionViewMode() == EClothPatternVertexType::Sim2D;
			}
			return false;
		}));


	CommandList->MapAction(
		CommandInfos.SetConstructionMode3D,
		FExecuteAction::CreateLambda([this]()
		{
			if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode()) 
			{
				ClothEdMode->SetConstructionViewMode(EClothPatternVertexType::Sim3D);
			}
		}),
		FCanExecuteAction::CreateLambda([this]() 
		{ 
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanChangeConstructionViewModeTo(EClothPatternVertexType::Sim3D);
			}
			return false; 
		}),
		FIsActionChecked::CreateLambda([this]() 
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->GetConstructionViewMode() == EClothPatternVertexType::Sim3D;
			}
			return false;
		}));



	CommandList->MapAction(
		CommandInfos.SetConstructionModeRender,
		FExecuteAction::CreateLambda([this]()
		{
			if (UChaosClothAssetEditorMode* ClothEdMode = GetEdMode()) 
			{
				ClothEdMode->SetConstructionViewMode(EClothPatternVertexType::Render);
			}
		}),
		FCanExecuteAction::CreateLambda([this]() 
		{ 
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanChangeConstructionViewModeTo(EClothPatternVertexType::Render);
			}
			return false; 
		}),
		FIsActionChecked::CreateLambda([this]() 
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->GetConstructionViewMode() == EClothPatternVertexType::Render;
			}
			return false;
		}));

	CommandList->MapAction(
		CommandInfos.ToggleConstructionViewWireframe,
		FExecuteAction::CreateLambda([this]()
		{
			const FEditorModeTools* const EditorModeTools = Client->GetModeTools();
			UChaosClothAssetEditorMode* const ClothEdMode = Cast<UChaosClothAssetEditorMode>(EditorModeTools->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId));

			if (ClothEdMode)
			{
				ClothEdMode->ToggleConstructionViewWireframe();
			}
		}),
		FCanExecuteAction::CreateLambda([this]() 
		{ 
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanSetConstructionViewWireframeActive();
			}
			return false;
		}),
		FIsActionChecked::CreateLambda([this]() 
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->IsConstructionViewWireframeActive();
			}
			return false;
		}));

	CommandList->MapAction(
		CommandInfos.ToggleConstructionViewSeams,
		FExecuteAction::CreateLambda([this]()
		{
			if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				ClothEdMode->ToggleConstructionViewSeams();
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanSetConstructionViewSeamsActive();
			}
			return false;
		}),
		FIsActionChecked::CreateLambda([this]()
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->IsConstructionViewSeamsActive();
			}
			return false;
		}));

	CommandList->MapAction(
		CommandInfos.ToggleConstructionViewSeamsCollapse,
		FExecuteAction::CreateLambda([this]()
		{
			if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				ClothEdMode->ToggleConstructionViewSeamsCollapse();
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->CanSetConstructionViewSeamsCollapse();
			}
			return false;
		}),
		FIsActionChecked::CreateLambda([this]()
		{
			if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
			{
				return ClothEdMode->IsConstructionViewSeamsCollapseActive();
			}
			return false;
		}));

	CommandList->MapAction(
		CommandInfos.TogglePatternColor,
		FExecuteAction::CreateLambda([this]()
			{
				if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					ClothEdMode->TogglePatternColor();
				}
			}),
		FCanExecuteAction::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->CanSetPatternColor();
				}
				return false;
			}),
		FIsActionChecked::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->IsPatternColorActive();
				}
				return false;
			}));

	CommandList->MapAction(
		CommandInfos.ToggleMeshStats,
		FExecuteAction::CreateLambda([this]()
			{
				if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					ClothEdMode->ToggleMeshStats();
				}
			}),
		FCanExecuteAction::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->CanSetMeshStats();
				}
				return false;
			}),
		FIsActionChecked::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->IsMeshStatsActive();
				}
				return false;
			}));

	CommandList->MapAction(
		CommandInfos.ToggleConstructionViewSurfaceNormals,
		FExecuteAction::CreateLambda([this]()
			{
				if (UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					ClothEdMode->ToggleConstructionViewSurfaceNormals();
				}
			}),
		FCanExecuteAction::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->CanSetConstructionViewSurfaceNormalsActive();
				}
				return false;
			}),
		FIsActionChecked::CreateLambda([this]()
			{
				if (const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode())
				{
					return ClothEdMode->IsConstructionViewSurfaceNormalsActive();
				}
				return false;
			}));


}

TSharedPtr<SWidget> SChaosClothAssetEditorRestSpaceViewport::BuildViewportToolbar()
{
	using namespace UE::Chaos::ClothAsset;

	const FName ToolbarName = "ChaosClothEditor.RestSpaceViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolbarMenu->StyleName = "ViewportToolbar";
		
		ToolbarMenu->AddSection("Left");
		
		{
			FToolMenuSection& RightSection = ToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
			
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
			
			{
				// View Modes
				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
				UToolMenu* ViewModesMenu = UToolMenus::Get()->ExtendMenu(UToolMenus::JoinMenuPaths(ToolbarName, "ViewModes"));
				FToolMenuSection& ViewSection = ViewModesMenu->FindOrAddSection("Cloth", LOCTEXT("ClothViewModeSection", "Cloth"));
				
				FToolMenuEntry& WireframeEntry = ViewSection.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleConstructionViewWireframe);
				WireframeEntry.SetShowInToolbarTopLevel(true);
				WireframeEntry.ToolBarData.ResizeParams.ClippingPriority = 2000;
				
				ViewSection.AddEntry(CreateDynamicLightIntensityItem());
			}
			
			{
				// Show Menu
				RightSection.AddEntry(UE::UnrealEd::CreateShowSubmenu(
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
					{
						FToolMenuSection& Section = InMenu->FindOrAddSection("Cloth", LOCTEXT("ClothShowMenu", "Chaos Cloth"));
						Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleMeshStats);
						Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleConstructionViewSeams);
						Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleConstructionViewSeamsCollapse);
						Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().TogglePatternColor);
						Section.AddMenuEntry(FChaosClothAssetEditorCommands::Get().ToggleConstructionViewSurfaceNormals);
					})
				));
			}
			
			RightSection.AddEntry(CreateDynamicSimulationMenuItem());
		}
	}
	
	FToolMenuContext Context;
	{
		Context.AppendCommandList(GetCommandList());
		Context.AddExtender(GetExtenders());
		
		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		Context.AddObject(ContextObject);
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

void SChaosClothAssetEditorRestSpaceViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
	.Padding(FMargin(4.f, 4.f, 0.f, 0.f))
	[
		SNew(SRichTextBlock)
		.DecoratorStyleSet(&FAppStyle::Get())
		.Text(this, &SChaosClothAssetEditorRestSpaceViewport::GetDisplayString)
		.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimViewport.MessageText"))
	];
}

FText SChaosClothAssetEditorRestSpaceViewport::GetDisplayString() const
{
	if (const FEditorModeTools* const EditorModeTools = RestSpaceViewportClient->GetModeTools())
	{
		if (UChaosClothAssetEditorMode* const ClothEdMode = Cast<UChaosClothAssetEditorMode>(EditorModeTools->GetActiveScriptableMode(UChaosClothAssetEditorMode::EM_ChaosClothAssetEditorModeId)))
		{
			if (ClothEdMode->IsMeshStatsActive())
			{
				const int TriangleCount = ClothEdMode->GetConstructionViewTriangleCount();
				const int VertexCount = ClothEdMode->GetConstructionViewVertexCount();
				const FText MeshStats = FText::Format(LOCTEXT("RestSpaceMeshStats", "Tris: {0}, Verts: {1}"), TriangleCount, VertexCount);
				return MeshStats;
			}
		}
	}
	return FText();
}

void SChaosClothAssetEditorRestSpaceViewport::OnFocusViewportToSelection()
{
	const UChaosClothAssetEditorMode* const ClothEdMode = GetEdMode();

	if (ClothEdMode)
	{
		const FBox BoundingBox = ClothEdMode->SelectionBoundingBox();
		if (BoundingBox.IsValid && !(BoundingBox.Min == FVector::Zero() && BoundingBox.Max == FVector::Zero()))
		{
			Client->FocusViewportOnBox(BoundingBox);

			// Reset any changes to the clip planes by the scroll zoom behavior
			Client->OverrideNearClipPlane(UE_KINDA_SMALL_NUMBER);
			Client->OverrideFarClipPlane(0);
		}
	}
}

bool SChaosClothAssetEditorRestSpaceViewport::IsVisible() const
{
	// Intentionally not calling SEditorViewport::IsVisible because it will return false if our simulation is more than 250ms.
	return ViewportWidget.IsValid();
}

TSharedRef<class SEditorViewport> SChaosClothAssetEditorRestSpaceViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SChaosClothAssetEditorRestSpaceViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SChaosClothAssetEditorRestSpaceViewport::OnFloatingButtonClicked()
{
}

#undef LOCTEXT_NAMESPACE
