// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SBaseCharacterFXEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UChaosClothAssetEditorMode;
namespace UE::Chaos::ClothAsset { class FChaosClothEditorRestSpaceViewportClient; }

class SChaosClothAssetEditorRestSpaceViewport : public SBaseCharacterFXEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:

	SLATE_BEGIN_ARGS(SChaosClothAssetEditorRestSpaceViewport) {}
		SLATE_ATTRIBUTE(FVector2D, ViewportSize);
		SLATE_ARGUMENT(TSharedPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient>, RestSpaceViewportClient)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportConstructionArgs);

	// SEditorViewport
	UE_API virtual void BindCommands() override;
	UE_API virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	UE_API virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	UE_API virtual void OnFocusViewportToSelection() override;
	UE_API virtual bool IsVisible() const override;

	// ICommonEditorViewportToolbarInfoProvider
	UE_API virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	UE_API virtual TSharedPtr<FExtender> GetExtenders() const override;
	UE_API virtual void OnFloatingButtonClicked() override;

private:

	UE_API UChaosClothAssetEditorMode* GetEdMode() const;
	
	UE_API FText GetDisplayString() const; 

	TSharedPtr<UE::Chaos::ClothAsset::FChaosClothEditorRestSpaceViewportClient> RestSpaceViewportClient;

};

#undef UE_API
