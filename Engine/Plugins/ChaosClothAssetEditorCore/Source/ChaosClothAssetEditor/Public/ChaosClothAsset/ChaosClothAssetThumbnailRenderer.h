// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "This header and its internal counterpart are scheduled for removal. Do not include either version.")

#include "ChaosClothAssetEditor/Internal/ChaosClothAsset/ChaosClothAssetThumbnailRenderer.h"
