// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CHAOSCLOTHASSETEDITOR_API

class UChaosClothAsset;
class UClothingAssetCommon;

namespace UE::Chaos::ClothAsset
{
	/** Result of a legacy-to-ChaosClothAsset conversion. */
	struct FLegacyClothingConverterResult
	{
		/** Newly created cloth asset; nullptr on failure. */
		UChaosClothAsset* CreatedAsset = nullptr;
		/** Object path of the created asset, or empty on failure. */
		FString CreatedAssetPath;
		/** Populated on failure with the user-visible error. */
		FText ErrorText;
	};

	/**
	 * Converts a legacy UClothingAssetCommon (the cloth asset driven by UChaosClothingSimulationFactory)
	 * into a new UChaosClothAsset built from the DF_LegacyClothingAssetTemplate Dataflow template.
	 *
	 * The new asset's Dataflow graph is baked from the legacy values so it is directly editable as if
	 * authored from scratch:
	 *  - Geometry is one-shot baked: a cloth collection built from the legacy FClothPhysicalMeshData is
	 *    stored on the new asset as the "ImportedSimClothCollection" Dataflow variable override (and the
	 *    legacy asset's owning SkeletalMesh as "RenderSkeletalMesh"). The template consumes these via a
	 *    Get-Variable node -- there is no per-asset import node and no live reference back to the source
	 *    legacy asset, so the conversion is not re-evaluated on Reimport.
	 *  - One WeightMapNode is appended per legacy weight map (values rescaled to [0,1] with the consuming
	 *    Simulation node's Low/High adjusted by LegacyMax).
	 *  - When the legacy asset has a TetherEndsMask weight map, LongRangeAttachmentsNode is switched into custom-tether
	 *    generation: two SelectionNodes ("LegacyTetherEnds" for the fixed-end set, "LegacyAllSimVertices3D"
	 *    for the dynamic-end set) are spliced upstream of LongRangeAttachmentsNode and wired into CustomTetherData[0]. When
	 *    no TetherEndsMask is present, the template's default wiring (FixedEndSet =
	 *    MaxDistanceConfig.KinematicVertices3D) is kept.
	 *  - Scalar UChaosClothConfig / UChaosClothSharedSimConfig values are written onto the existing
	 *    Simulation* config nodes via UProperty reflection.
	 *
	 * TODO: multi-LOD -- only clothing LOD 0 is imported. Legacy assets with multiple LODs lose all
	 * but LOD 0 on conversion. The new asset's Dataflow graph has no multi-LOD authoring yet either.
	 */
	UE_EXPERIMENTAL(5.8, "This converter is experimental. Use at your own risk!")
	class FLegacyClothingConverter
	{
	public:
		/**
		 * Create a new UChaosClothAsset at OutputPackagePath/AssetName from the legacy template and
		 * bake the legacy values onto it.
		 */
		static UE_API FLegacyClothingConverterResult Convert(
			const UClothingAssetCommon* SourceAsset,
			const FString& OutputPackagePath,
			const FString& AssetName);

		/**
		 * Convert into an existing UChaosClothAsset (mutating it in place). The target's embedded
		 * Dataflow is reset to a duplicate of the legacy conversion template, then all legacy values
		 * are baked onto the graph + variable overrides as in Convert().
		 */
		static UE_API FLegacyClothingConverterResult ConvertInto(
			const UClothingAssetCommon* SourceAsset,
			UChaosClothAsset* TargetAsset);
	};
}

#undef UE_API
