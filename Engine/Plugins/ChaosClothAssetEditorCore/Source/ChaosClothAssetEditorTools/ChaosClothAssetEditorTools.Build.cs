// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetEditorTools : ModuleRules
{
	public ChaosClothAssetEditorTools(ReadOnlyTargetRules Target) : base(Target)
	{	
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"UnrealEd",
				"InputCore",
				"InteractiveToolsFramework",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",
				"Slate",
				"SlateCore",
				"Projects",
				"AdvancedPreviewScene",
				"ContentBrowser",
				"DynamicMesh",
				"EditorFramework", // FEditorModeInfo
				"EditorInteractiveToolsFramework",
				"ModelingComponentsEditorOnly", // Static/skeletal mesh tool targets
				"ModelingComponents",
				"MeshModelingTools",
				"MeshModelingToolsExp",
				"EditorStyle",
				"EditorSubsystem",
				"GeometryCore",
				"GeometryFramework",
				"LevelEditor",
				"StatusBar",
				"ToolMenus",
				"ToolWidgets",
				"WorkspaceMenuStructure",
				"ClothingSystemRuntimeCommon",
				"ClothingSystemEditorInterface",
				"ClothingSystemRuntimeInterface",
				"ChaosCloth",
				"ChaosClothAssetTools",
				"ChaosClothAssetEngine",
				"ClothingSystemEditor",
				"MeshConversion",
				"MeshDescription",
				"SkeletalMeshDescription",
				"ChaosClothAsset",
				"ChaosClothAssetDataflowNodes",
				"DataflowCore",
				"DataflowEngine",
				"DataflowEditor",
				"Chaos"				// FManagedArrayCollection::StaticType
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
