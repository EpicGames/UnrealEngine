// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "Dataflow/DataflowToolRegistry.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "DataflowEditorTools/DataflowEditorVertexAttributePaintTool.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothEditorToolBuilders.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "ChaosClothAsset/ClothToolActionCommandBindings.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetEditorToolsModule"

namespace UE::Chaos::ClothAsset
{
	class FChaosClothAssetEditorToolsModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
		
			const TSharedRef<const FClothToolActionCommandBindings> ClothToolActions = MakeShared<FClothToolActionCommandBindings>();

			UClothEditorWeightMapPaintToolBuilder* FallbackBuilder = NewObject<UClothEditorWeightMapPaintToolBuilder>();
			UDataflowEditorVertexAttributePaintToolBuilder* WeightMapBuilder = NewObject<UDataflowEditorVertexAttributePaintToolBuilder>();
			WeightMapBuilder->SetFallbackToolBuilder(FallbackBuilder);
			
			ToolRegistry.AddNodeToToolMapping(FChaosClothAssetWeightMapNode::StaticType(),          // Node type
				WeightMapBuilder,                                 // Tool builder
				ClothToolActions,                                                                   // Tool action bindings
				FSlateIcon(FName("ClothStyle"), FName("ChaosClothAssetEditor.AddWeightMapNode")),   // Button icon (see FChaosClothAssetEditorStyle)
				LOCTEXT("AddWeightMapNodeButtonText", "Cloth Weight Map"), FName("Cloth"));                               // Button text

			ToolRegistry.AddNodeToToolMapping(FChaosClothAssetSelectionNode_v2::StaticType(), 
				NewObject<UClothMeshSelectionToolBuilder>(), 
				ClothToolActions, 
				FSlateIcon(FName("ClothStyle"), FName("ChaosClothAssetEditor.AddMeshSelectionNode")),
				LOCTEXT("AddSelectionNodeButtonText", "Cloth Mesh Selection"), FName("Cloth"));

			ToolRegistry.AddNodeToToolMapping(FChaosClothAssetTransferSkinWeightsNode::StaticType(), 
				NewObject<UClothTransferSkinWeightsToolBuilder>(), 
				ClothToolActions, 
				FSlateIcon(FName("ClothStyle"), FName("ChaosClothAssetEditor.AddTransferSkinWeightsNode")),
				LOCTEXT("AddTransferSkinWeightNodeButtonText", "Cloth Skinning Transfer"), FName("Cloth"));

			const FSlateIcon ChaosClothAssetIcon("ClothAssetEditorStyle", "ClassThumbnail.ChaosClothAsset");

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosClothAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosClothAsset/DF_EmptyClothAssetTemplate.DF_EmptyClothAssetTemplate")),
					.DisplayName = LOCTEXT("SelectEmptyTemplate", "Empty Dataflow"),
					.Tooltip = LOCTEXT("SelectEmptyTemplateTooltip", "Add an empty Dataflow graph with a Cloth Asset Terminal node.\n(Requires the ChaosClothAssetDataflowNodes plug-in.)"),
					 .Icon = ChaosClothAssetIcon
				});

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosClothAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosClothAsset/DF_StaticMeshClothTemplate.DF_StaticMeshClothTemplate")),
					.DisplayName = LOCTEXT("SelectStaticMeshClothTemplate", "Static Mesh Cloth"),
					.Tooltip = LOCTEXT("SelectStaticMeshClothTemplateTooltip", "Add a Dataflow graph that builds a Cloth Asset from Static Meshes.\n(Requires the ChaosClothAssetDataflowNodes plug-in.)"),
					 .Icon = ChaosClothAssetIcon
				});
			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosClothAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosClothAsset/DF_SkeletalMeshClothTemplate.DF_SkeletalMeshClothTemplate")),
					.DisplayName = LOCTEXT("SelectSkeletalMeshClothTemplate", "Skeletal Mesh Cloth"),
					.Tooltip = LOCTEXT("SelectSkeletalMeshClothTemplateTooltip", "Add a Dataflow graph that builds a Cloth Asset from Skeletal Meshes.\n(Requires the ChaosClothAssetDataflowNodes plug-in.)"),
					 .Icon = ChaosClothAssetIcon
				});
			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosClothAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosClothAsset/ClothAssetTemplate.ClothAssetTemplate")),
					.DisplayName = LOCTEXT("SelectUsdClothTemplate", "USD Cloth"),
					.Tooltip = LOCTEXT("SelectUsdClothTemplateTooltip", "Add a Dataflow graph that builds a Cloth Asset from an imported USD Cloth.\n(Requires the ChaosClothAssetDataflowNodes and ChaosClothAssetUsdDataflowNodes plug-ins.)"),
					 .Icon = ChaosClothAssetIcon
				});
		}

		virtual void ShutdownModule() override
		{
			UE::Dataflow::FDataflowToolRegistry& ToolRegistry = UE::Dataflow::FDataflowToolRegistry::Get();
			ToolRegistry.RemoveNodeToToolMapping(FChaosClothAssetWeightMapNode::StaticType());
			ToolRegistry.RemoveNodeToToolMapping(FChaosClothAssetSelectionNode_v2::StaticType());
			ToolRegistry.RemoveNodeToToolMapping(FChaosClothAssetTransferSkinWeightsNode::StaticType());
		}
	};
}

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FChaosClothAssetEditorToolsModule, ChaosClothAssetEditorTools)

#undef LOCTEXT_NAMESPACE
