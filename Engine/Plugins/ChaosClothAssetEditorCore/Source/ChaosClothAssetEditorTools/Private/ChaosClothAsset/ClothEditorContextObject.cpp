// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorContextObject.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorContextObject)

void UClothEditorContextObject::Init(TWeakPtr<SDataflowGraphEditor> InDataflowGraphEditor, TWeakPtr<UE::Dataflow::FEngineContext> InDataflowContext, UE::Chaos::ClothAsset::EClothPatternVertexType InConstructionViewMode, TWeakPtr<FManagedArrayCollection> InSelectedClothCollection, bool bInUsingInputCollection)
{
	DataflowGraphEditor = InDataflowGraphEditor;
	DataflowContext = InDataflowContext;
	ConstructionViewMode = InConstructionViewMode;
	SelectedClothCollection = InSelectedClothCollection;
	bUsingInputCollection = bInUsingInputCollection;
}

void UClothEditorContextObject::SetClothCollection(UE::Chaos::ClothAsset::EClothPatternVertexType ViewMode, TWeakPtr<FManagedArrayCollection> ClothCollection, bool bInUsingInputCollection)
{
	ConstructionViewMode = ViewMode;
	SelectedClothCollection = ClothCollection;
	bUsingInputCollection = bInUsingInputCollection;
}