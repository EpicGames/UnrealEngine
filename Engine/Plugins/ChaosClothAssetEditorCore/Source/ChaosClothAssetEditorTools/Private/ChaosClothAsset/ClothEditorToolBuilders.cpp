// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothEditorToolBuilders.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "ToolContextInterfaces.h"
#include "ToolTargetManager.h"
#include "ContextObjectStore.h"
#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowRenderingViewMode.h"

// Tools
#include "ClothMeshSelectionTool.h"
#include "ClothTransferSkinWeightsTool.h"
#include "ClothWeightMapPaintTool.h"


// ------------------- Weight Map Paint Tool -------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothEditorToolBuilders)

void UClothEditorWeightMapPaintToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	using namespace UE::Chaos::ClothAsset;

	const UE::Dataflow::FRenderingViewModeFactory& Factory = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
	const UE::Dataflow::IDataflowConstructionViewMode* const Sim2DMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Sim2D));
	const UE::Dataflow::IDataflowConstructionViewMode* const Sim3DMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Sim3D));
	const UE::Dataflow::IDataflowConstructionViewMode* const RenderMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Render));

	checkf(Sim2DMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Sim2D"));
	checkf(Sim3DMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Sim3D"));
	checkf(RenderMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Render"));

	const FChaosClothAssetWeightMapNode* const WeightMapNode = ContextObject.GetSelectedNodeOfType<FChaosClothAssetWeightMapNode>();
	if (WeightMapNode)
	{
		if (WeightMapNode->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render)
		{
			Modes.Add(RenderMode);
		}
		else
		{
			check(WeightMapNode->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation);
			Modes.Add(Sim2DMode);
			Modes.Add(Sim3DMode);
		}
	}
	else
	{
		const EClothPatternVertexType CurrentViewMode = DataflowViewModeToClothViewMode(ContextObject.GetConstructionViewMode());

		if (CurrentViewMode == EClothPatternVertexType::Render)
		{
			Modes.Add(RenderMode);
		}
		else
		{
			Modes.Add(Sim2DMode);
			Modes.Add(Sim3DMode);
		}
	}
}

bool UClothEditorWeightMapPaintToolBuilder::CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const
{
	return ActiveTool->IsA<UClothEditorWeightMapPaintTool>();
}

void UClothEditorWeightMapPaintToolBuilder::SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState)
{
	check(CanSceneStateChange(ActiveTool, SceneState));

	UClothEditorWeightMapPaintTool* const PaintTool = Cast<UClothEditorWeightMapPaintTool>(ActiveTool);
	checkf(PaintTool, TEXT("Expected the ActiveTool to be UClothEditorWeightMapPaintTool"));

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	check(Target->IsValid());
	PaintTool->SetTarget(Target);
	PaintTool->NotifyTargetChanged();

	// These are likely to be empty functions but are called here for completeness (see UInteractiveToolManager::ActivateToolInternal())
	PostBuildTool(ActiveTool, SceneState);
	PostSetupTool(ActiveTool, SceneState);
}

void UClothEditorWeightMapPaintToolBuilder::GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const
{
	using namespace UE::Chaos::ClothAsset;
	const FChaosClothAssetWeightMapNode* const WeightMapNode = ContextObject.GetSelectedNodeOfType<FChaosClothAssetWeightMapNode>();
	if (WeightMapNode)
	{
		if (WeightMapNode->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
		{
			Modes.Add(EClothPatternVertexType::Sim3D);
			Modes.Add(EClothPatternVertexType::Sim2D);
		}
		else
		{
			check(WeightMapNode->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render);
			Modes.Add(EClothPatternVertexType::Render);
		}
	}
	else
	{
		// No node selected. This happens if we start the tool due to pushing the button in the toolbar -- the tool starts before the node selection can change.
		// In this case lock to either sim or render mode, whatever is current.
		// TODO: See if we can have the button action select the node before attempting to start the tool.
		
		const EClothPatternVertexType ViewMode = DataflowViewModeToClothViewMode(ContextObject.GetConstructionViewMode());
		const bool bViewModeIsRender = (ViewMode == EClothPatternVertexType::Render);
		
		if (bViewModeIsRender)
		{
			Modes.Add(EClothPatternVertexType::Render);
		}
		else
		{
			Modes.Add(EClothPatternVertexType::Sim3D);
			Modes.Add(EClothPatternVertexType::Sim2D);
		}
	}
}

bool UClothEditorWeightMapPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState))
	{
		if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
		{
			return (DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetWeightMapNode>() != nullptr);
		}
	}
	return false;
}

UMeshSurfacePointTool* UClothEditorWeightMapPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UClothEditorWeightMapPaintTool* PaintTool = NewObject<UClothEditorWeightMapPaintTool>(SceneState.ToolManager);
	PaintTool->SetWorld(SceneState.World);

	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		PaintTool->SetDataflowContextObject(DataflowContextObject);
	}

	return PaintTool;
}


// ------------------- Selection Tool -------------------


void UClothMeshSelectionToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	using namespace UE::Chaos::ClothAsset;

	const UE::Dataflow::FRenderingViewModeFactory& Factory = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
	const UE::Dataflow::IDataflowConstructionViewMode* const Sim2DMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Sim2D));
	const UE::Dataflow::IDataflowConstructionViewMode* const Sim3DMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Sim3D));
	const UE::Dataflow::IDataflowConstructionViewMode* const RenderMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Render));

	checkf(Sim2DMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Sim2D"));
	checkf(Sim3DMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Sim3D"));
	checkf(RenderMode, TEXT("Couldn't find DataflowConstructionViewMode corresponding to EClothPatternVertexType::Render"));

	const FChaosClothAssetSelectionNode_v2* const SelectionNode = ContextObject.GetSelectedNodeOfType<FChaosClothAssetSelectionNode_v2>();
	if (SelectionNode)
	{
		if (SelectionNode->Group.Name == ClothCollectionGroup::RenderVertices.ToString() || SelectionNode->Group.Name == ClothCollectionGroup::RenderFaces.ToString())
		{
			Modes.Add(RenderMode);
			return;
		}
		
		if (SelectionNode->Group.Name == ClothCollectionGroup::SimVertices2D.ToString())
		{
			Modes.Add(Sim2DMode);
			return;
		}

		if (SelectionNode->Group.Name == ClothCollectionGroup::SimVertices3D.ToString())
		{
			Modes.Add(Sim3DMode);
			return;
		}

		if (SelectionNode->Group.Name == ClothCollectionGroup::SimFaces.ToString())
		{
			Modes.Add(Sim2DMode);
			Modes.Add(Sim3DMode);
			return;
		}
	}

	// No node selected or no valid group name set in the node -- use the current view mode to decide

	const EClothPatternVertexType CurrentViewMode = DataflowViewModeToClothViewMode(ContextObject.GetConstructionViewMode());

	if (CurrentViewMode == EClothPatternVertexType::Render)
	{
		Modes.Add(RenderMode);
	}
	else
	{
		Modes.Add(Sim2DMode);
		Modes.Add(Sim3DMode);
	}
}

bool UClothMeshSelectionToolBuilder::CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const
{
	return ActiveTool->IsA<UClothMeshSelectionTool>();
}

void UClothMeshSelectionToolBuilder::SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState)
{
	check(CanSceneStateChange(ActiveTool, SceneState));

	UClothMeshSelectionTool* const SelectionTool = Cast<UClothMeshSelectionTool>(ActiveTool);
	checkf(SelectionTool, TEXT("Expected the ActiveTool to be UClothMeshSelectionTool"));

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	check(Target->IsValid());
	SelectionTool->SetTarget(Target);
	check(SelectionTool->GetTargetWorld() == SceneState.World);
	SelectionTool->NotifyTargetChanged();

	// These are likely to be empty functions but are called here for completeness (see UInteractiveToolManager::ActivateToolInternal())
	PostBuildTool(ActiveTool, SceneState);
	PostSetupTool(ActiveTool, SceneState);

}

void UClothMeshSelectionToolBuilder::GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const
{
	// TODO: When the Secondary Selection set is removed, update this function to be similar to UClothEditorWeightMapPaintToolBuilder::GetSupportedViewModes above
	Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D);
	Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Sim2D);
	Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Render);
}


const FToolTargetTypeRequirements& UClothMeshSelectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements(UPrimitiveComponentBackedTarget::StaticClass());
	return TypeRequirements;
}

bool UClothMeshSelectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		return DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetSelectionNode_v2>() != nullptr && (SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1);
	}
	return false;
}

UInteractiveTool* UClothMeshSelectionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UClothMeshSelectionTool* const NewTool = NewObject<UClothMeshSelectionTool>(SceneState.ToolManager);

	UToolTarget* const Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTarget(Target);
	NewTool->SetWorld(SceneState.World);

	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		NewTool->SetDataflowContextObject(DataflowContextObject);
	}

	return NewTool;
}


// ------------------- Skin Weight Transfer Tool -------------------


void UClothTransferSkinWeightsToolBuilder::GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const
{
	using namespace UE::Chaos::ClothAsset;
	const UE::Dataflow::FRenderingViewModeFactory& Factory = UE::Dataflow::FRenderingViewModeFactory::GetInstance();
	const UE::Dataflow::IDataflowConstructionViewMode* const Sim3DMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Sim3D));
	const UE::Dataflow::IDataflowConstructionViewMode* const RenderMode = Factory.GetViewMode(ClothViewModeToDataflowViewModeName(EClothPatternVertexType::Render));

	const FChaosClothAssetTransferSkinWeightsNode* const TransferNode = ContextObject.GetSelectedNodeOfType<FChaosClothAssetTransferSkinWeightsNode>();
	if (TransferNode)
	{
		switch (TransferNode->TargetMeshType)
		{
		case EChaosClothAssetTransferTargetMeshType::All:
			Modes.Add(Sim3DMode);
			Modes.Add(RenderMode);
			break;

		case EChaosClothAssetTransferTargetMeshType::Simulation:
			Modes.Add(Sim3DMode);
			break;

		case EChaosClothAssetTransferTargetMeshType::Render:
			Modes.Add(RenderMode);
			break;
		}
	}
}

bool UClothTransferSkinWeightsToolBuilder::CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const
{
	return false;
}

void UClothTransferSkinWeightsToolBuilder::SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState)
{
	check(CanSceneStateChange(ActiveTool, SceneState));
}


void UClothTransferSkinWeightsToolBuilder::GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const
{
	const FChaosClothAssetTransferSkinWeightsNode* const TransferNode = ContextObject.GetSelectedNodeOfType<FChaosClothAssetTransferSkinWeightsNode>();
	if (TransferNode)
	{
		switch (TransferNode->TargetMeshType)
		{
		case EChaosClothAssetTransferTargetMeshType::All:
			Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D);
			Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Render);
			break;

		case EChaosClothAssetTransferTargetMeshType::Simulation:
			Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Sim3D);
			break;

		case EChaosClothAssetTransferTargetMeshType::Render:
			Modes.Add(UE::Chaos::ClothAsset::EClothPatternVertexType::Render);
			break;
		}
	}
}

USingleSelectionMeshEditingTool* UClothTransferSkinWeightsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UClothTransferSkinWeightsTool* NewTool = NewObject<UClothTransferSkinWeightsTool>(SceneState.ToolManager);

	if (UDataflowContextObject* const DataflowContextObject = SceneState.ToolManager->GetContextObjectStore()->FindContext<UDataflowContextObject>())
	{
		NewTool->SetDataflowEditorContextObject(DataflowContextObject);
	}

	return NewTool;
}



namespace UE::Chaos::ClothAsset
{
	void GetClothEditorToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
	{
		ToolCDOs.Add(GetMutableDefault<UClothEditorWeightMapPaintTool>());
		ToolCDOs.Add(GetMutableDefault<UClothTransferSkinWeightsTool>());
		ToolCDOs.Add(GetMutableDefault<UClothMeshSelectionTool>());
	}

	EClothPatternVertexType DataflowViewModeToClothViewMode(const UE::Dataflow::IDataflowConstructionViewMode* DataflowViewMode)
	{
		const FName ViewModeName = DataflowViewMode->GetName();
		if (ViewModeName == FName("Cloth2DSimView"))
		{
			return EClothPatternVertexType::Sim2D;
		}
		else if (ViewModeName == FName("Cloth3DSimView"))
		{
			return EClothPatternVertexType::Sim3D;
		}
		else
		{
			// We should nromally expect ClothRenderView but in dataflow Editor this can be another mode 
			ensure(ViewModeName == FName("ClothRenderView"));
			if (DataflowViewMode->IsPerspective())
			{
				return EClothPatternVertexType::Render; // could be Sim3D but need to make a choice :)
			}
			return EClothPatternVertexType::Sim2D;
		}
	}

	FName ClothViewModeToDataflowViewModeName(EClothPatternVertexType ClothViewMode)
	{
		switch (ClothViewMode)
		{
		case EClothPatternVertexType::Sim2D:
			return FName("Cloth2DSimView");
		case EClothPatternVertexType::Sim3D:
			return FName("Cloth3DSimView");
		case EClothPatternVertexType::Render:
			return FName("ClothRenderView");
		default:
			checkNoEntry();
			return NAME_None;
		};
	}
}
