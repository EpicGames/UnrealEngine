// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothMeshSelectionTool.h"
#include "ChaosClothAsset/ClothEditorToolBuilders.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "InteractiveToolManager.h"
#include "PreviewMesh.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ModelingToolTargetUtil.h"
#include "GroupTopology.h"
#include "ToolSetupUtil.h"
#include "Selections/GeometrySelection.h"
#include "ContextObjectStore.h"
#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Materials/Material.h"
#include "Selections/MeshConnectedComponents.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "UnrealEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothMeshSelectionTool)

#define LOCTEXT_NAMESPACE "ClothMeshSelectionTool"


// ------------------- Actions ----------------------
void UClothMeshSelectionToolActions::PostAction(EClothMeshSelectionToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}

// ------------------- Properties -------------------

void UClothMeshSelectionToolProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UClothMeshSelectionToolProperties, Name))
	{
		UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Name);
	}
}


// ------------------- Selection Mechanic -------------------

bool UClothMeshSelectionMechanic::UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut)
{
	using namespace UE::Geometry;

	if (bShiftToggle && bCtrlToggle)		// TODO: Shift + Ctrl means something different when marquee is active
	{
		FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin), TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
		UE::Geometry::Normalize(LocalRay.Direction);

		const FGroupTopologySelection PreviousSelection = PersistentSelection;

		FVector3d LocalPosition, LocalNormal;
		FGroupTopologySelection Selection;
		const FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
		if (TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal))
		{
			LocalHitPositionOut = LocalPosition;
			LocalHitNormalOut = LocalNormal;

			// Get seed selection

			const FTopologyProvider* const TopologyProvider = TopoSelector->GetTopologyProvider();

			if (Properties->bSelectFaces && Selection.SelectedGroupIDs.Num() > 0)
			{
				// TopologyProvider doesn't provide an interface to get triangle indices from GroupIDs, so we ray cast again
				// TODO: Implement GroupID->Triangles function in FTopologyProvider, similar to GetCornerVertexID()

				FDynamicMeshAABBTree3* const Spatial = GetSpatialFunc();
				check(Spatial);
				const int32 TriangleID = Spatial->FindNearestHitTriangle(LocalRay);
				if (TriangleID != IndexConstants::InvalidID)
				{
					TSet<int32> ConnectedTriangles;
					FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, { TriangleID }, ConnectedTriangles);

					for (const int32 ConnectedTriangleID : ConnectedTriangles)
					{
						const int32 GroupID = TopologyProvider->GetGroupIDForTriangle(ConnectedTriangleID);
						if (TopologyProvider->GetGroupIDAt(GroupID) != -1)
						{
							PersistentSelection.SelectedGroupIDs.Remove(GroupID);
						}
					}
				}
			}
			else if (!Properties->bSelectFaces && Selection.SelectedCornerIDs.Num() > 0)
			{
				const int32 CornerID = *Selection.SelectedCornerIDs.CreateConstIterator();
				const int32 VertexID = TopologyProvider->GetCornerVertexID(CornerID);

				TSet<int32> ConnectedVertices;
				FMeshConnectedComponents::GrowToConnectedVertices(*Mesh, { VertexID }, ConnectedVertices);

				TArray<int32> CornersToRemove;

				// FGroupTopology has GetCornerIDFromVertexID() but is inaccessible
				// TODO: Expose Vertex -> CornerID map in FTopologyProvider

				for (const int32 ConnectedVertex : ConnectedVertices)
				{
					for (const int32 SelectedCornerID : PersistentSelection.SelectedCornerIDs)
					{
						if (TopologyProvider->GetCornerVertexID(SelectedCornerID) == ConnectedVertex)
						{
							CornersToRemove.Add(SelectedCornerID);
							break;
						}
					}
				}
				for (const int32 Remove : CornersToRemove)
				{
					PersistentSelection.SelectedCornerIDs.Remove(Remove);
				}
			}

			if (PersistentSelection != PreviousSelection)
			{
				SelectionTimestamp++;
				OnSelectionChanged.Broadcast();
				return true;
			}
		}
	}
	else
	{
		return UPolygonSelectionMechanic::UpdateSelection(WorldRay, LocalHitPositionOut, LocalHitNormalOut);
	}

	return false;
}


// ------------------- Tool -------------------
	

void UClothMeshSelectionTool::InitializeSculptMeshFromTarget()
{
	using namespace UE::Geometry;

	//
	// Preview 
	//

	// (Re-)Create the preview mesh
	if (PreviewMesh)
	{
		PreviewMesh->Disconnect();
	}
	PreviewMesh = NewObject<UPreviewMesh>(this, TEXT("PreviewMesh"));
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);

	// We will use the preview mesh's spatial data structure
	PreviewMesh->bBuildSpatialDataStructure = true;

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	// configure secondary render material for selected triangles
	// NOTE: the material returned by ToolSetupUtil::GetSelectionMaterial has a checkerboard pattern on back faces which makes it hard to use
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial"));
	if (Material != nullptr)
	{
		if (UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, GetToolManager()))
		{
			MatInstance->SetVectorParameterValue(TEXT("Color"), FLinearColor::Yellow);
			PreviewMesh->SetSecondaryRenderMaterial(MatInstance);
		}
	}

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	PreviewMesh->UpdatePreview(UE::ToolTarget::GetDynamicMeshCopy(Target));
	PreviewMesh->SetVisible(true);

	// Hide input target mesh
	UE::ToolTarget::HideSourceObject(Target);

	// Setup non-manifold mapping if necessary
	if (DataflowContextObject)
	{
		ensure(DataflowContextObject->IsUsingInputCollection());
		if (const TSharedPtr<const FManagedArrayCollection> ClothCollection = DataflowContextObject->GetSelectedCollection())
		{
			PreviewMesh->ProcessMesh([this, ClothCollection](const FDynamicMesh3& Mesh)
			{
				using namespace UE::Chaos::ClothAsset;
				const UE::Geometry::FNonManifoldMappingSupport NonManifoldMapping(Mesh);

				bHasNonManifoldMapping = NonManifoldMapping.IsNonManifoldVertexInSource();

				if (bHasNonManifoldMapping)
				{
					const FCollectionClothConstFacade Cloth(ClothCollection.ToSharedRef());
					check(Cloth.IsValid());

					const TConstArrayView<int32> SimVertex3DLookup = Cloth.GetSimVertex3DLookup();

					DynamicMeshToSelection.SetNumUninitialized(Mesh.VertexCount());
					SelectionToDynamicMesh.Reset();
					SelectionToDynamicMesh.SetNum(Cloth.GetNumSimVertices3D());
					for (int32 DynamicMeshVert = 0; DynamicMeshVert < Mesh.VertexCount(); ++DynamicMeshVert)
					{
						DynamicMeshToSelection[DynamicMeshVert] = NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert);
						SelectionToDynamicMesh[DynamicMeshToSelection[DynamicMeshVert]].Add(DynamicMeshVert);
					}
				}
			});
		}
	}
}



void UClothMeshSelectionTool::Setup()
{
	using namespace UE::Geometry;

	InitializeSculptMeshFromTarget();

	//
	// SelectionMechanic
	//

	SelectionMechanic = NewObject<UClothMeshSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;   // We'll do this ourselves later
	SelectionMechanic->Setup(this);
	SelectionMechanic->Properties->RestoreProperties(this);
	SelectionMechanic->Properties->bDisplayPolygroupReliantControls = false;	// this is for polygroup-specific selections like edge loops
	
	SelectionMechanic->Properties->bCanSelectVertices = true;
	SelectionMechanic->Properties->bCanSelectEdges = false;		// For now do not allow edge selection
	SelectionMechanic->Properties->bCanSelectFaces = true;
	
	SelectionMechanic->SetShowEdges(false);
	SelectionMechanic->SetShowSelectableCorners(false);

	SelectionMechanic->PolyEdgesRenderer.DepthBias = 0.01f;
	SelectionMechanic->PolyEdgesRenderer.LineThickness = 1.0f;
	SelectionMechanic->PolyEdgesRenderer.PointSize = 2.0f;

	SelectionMechanic->SelectionRenderer.DepthBias = 0.01f;
	SelectionMechanic->SelectionRenderer.LineThickness = 1.0f;
	SelectionMechanic->SelectionRenderer.PointSize = 2.0f;

	SelectionMechanic->OnSelectionChanged.AddWeakLambda(this, [this]()
	{
		bAnyChangeMade = true;
		PreviewMesh->FastNotifySecondaryTrianglesChanged();
	});

	SelectionMechanic->OnFaceSelectionPreviewChanged.AddWeakLambda(this, [this]()
	{
		PreviewMesh->FastNotifySecondaryTrianglesChanged();
	});


	// Enable only one selection mode at a time (this is different from other mesh modeling tools using the SelectionMechanic)

	SelectionMechanic->Properties->WatchProperty(SelectionMechanic->Properties->bSelectVertices, [this](bool bNewSelectVertices)
	{
		SelectionMechanic->Properties->bSelectFaces = !bNewSelectVertices;
	});

	SelectionMechanic->Properties->WatchProperty(SelectionMechanic->Properties->bSelectFaces, [this](bool bNewSelectFaces)
	{
		SelectionMechanic->Properties->bSelectVertices = !bNewSelectFaces;
	});

	// Set up Topology and SelectionMechanic using Preview's DynamicMesh

	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		Topology = MakeUnique<FTriangleGroupTopology>(&Mesh, true);

		SelectionMechanic->Initialize(&Mesh,
			FTransform3d(),
			GetTargetWorld(),
			Topology.Get(),
			[this]() { return PreviewMesh->GetSpatial(); });
	});

	PreviewMesh->EnableSecondaryTriangleBuffers([this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
	});



	//
	// Properties
	//

	ToolProperties = NewObject<UClothMeshSelectionToolProperties>();
	ToolProperties->WatchProperty(ToolProperties->bShowVertices, [this](bool bNewShowVertices)
	{
		SelectionMechanic->SetShowSelectableCorners(bNewShowVertices);
	});

	ToolProperties->WatchProperty(ToolProperties->bShowEdges, [this](bool bNewShowEdges)
	{
		SelectionMechanic->SetShowEdges(bNewShowEdges);
	});

	// Order of operations is important here:
	// ToolProperties->WatchProperty(ToolProperties->Name) should happen after GetSelectedNodeInfo so that we can set the OriginalName

	ToolProperties->RestoreProperties(this);
	 
	// Initialize the Selection from the selected Dataflow node
	FString ExistingSelectionName;
	FGroupTopologySelection ExistingNodeSelection;
	EChaosClothAssetSelectionOverrideType ExistingOverrideType;
	GetSelectedNodeInfo(ExistingSelectionName, ExistingNodeSelection, ExistingOverrideType);

	constexpr bool bBroadcastChange = false;
	SelectionMechanic->SetSelection(ExistingNodeSelection, bBroadcastChange);

	if (ExistingNodeSelection.SelectedCornerIDs.Num() > 0 || 
		SelectionNodeToUpdate->Group.Name == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D || 
		SelectionNodeToUpdate->Group.Name == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices2D ||
		SelectionNodeToUpdate->Group.Name == UE::Chaos::ClothAsset::ClothCollectionGroup::RenderVertices)
	{
		check(ExistingNodeSelection.SelectedEdgeIDs.Num() == 0);
		check(ExistingNodeSelection.SelectedGroupIDs.Num() == 0);
		SelectionMechanic->Properties->bSelectVertices = true;
		SelectionMechanic->Properties->bSelectFaces = false;
	}
	else
	{
		SelectionMechanic->Properties->bSelectVertices = false;
		SelectionMechanic->Properties->bSelectFaces = true;
	}


    ToolProperties->Name = ExistingSelectionName;
	ToolProperties->WatchProperty(ToolProperties->Name, [this, OriginalName = ToolProperties->Name](const FString& NewName)
	{
		if (NewName != OriginalName)
		{
			bAnyChangeMade = true;
		}
	});
	ToolProperties->SelectionOverrideType = ExistingOverrideType;


	// 
	// Actions
	//

	ActionsProps = NewObject<UClothMeshSelectionToolActions>();
	ActionsProps->Initialize(this);
	AddToolPropertySource(ActionsProps);

	AddToolPropertySource(ToolProperties);

	AddToolPropertySource(SelectionMechanic->Properties);
}

void UClothMeshSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && CanAccept())
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionToolTransactionName", "Mesh Selection"));
		UpdateSelectedNode();
		GetToolManager()->EndUndoTransaction();

		SelectionNodeToUpdate->Invalidate();
	}

	SelectionMechanic->Properties->SaveProperties(this);
	ToolProperties->SaveProperties(this);

	if (PreviewMesh != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	SelectionMechanic->Shutdown();

	Topology.Reset();
}

void UClothMeshSelectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	SelectionMechanic->Render(RenderAPI);
}

void UClothMeshSelectionTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	SelectionMechanic->DrawHUD(Canvas, RenderAPI);

	if (Canvas && RenderAPI)
	{
		const UE::Geometry::FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();

		const UE::Geometry::FDynamicMesh3* Mesh = PreviewMesh ? PreviewMesh->GetPreviewDynamicMesh() : nullptr;

		const FText BaseText = LOCTEXT("UClothMeshSelectionTool_Info", "Selected : Vertices: {0} / {1} - Triangles: {2} / {3}");
		const FText InfoText = FText::Format(BaseText, 
			FText::AsNumber(Selection.SelectedCornerIDs.Num()),
			FText::AsNumber(Mesh? Mesh->VertexCount(): 0),
			FText::AsNumber(Selection.SelectedGroupIDs.Num()),
			FText::AsNumber(Mesh ? Mesh->TriangleCount() : 0)
		);
		const FVector2D TextPosition = { 0.5f * Canvas->GetViewRect().Width() * Canvas->GetDPIScale(), 10};
		FCanvasTextItem TextItem(TextPosition, InfoText, GEngine->GetLargeFont(), FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.bCentreX = true;
		TextItem.Scale = FVector2D(1.2);
		Canvas->DrawItem(TextItem);
	}
}

void UClothMeshSelectionTool::OnTick(float DeltaTime)
{
	USingleSelectionMeshEditingTool::OnTick(DeltaTime);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EClothMeshSelectionToolActions::NoAction;
	}
}

bool UClothMeshSelectionTool::CanAccept() const
{
	return bAnyChangeMade || SelectionNodeToUpdate->SelectionOverrideType != ToolProperties->SelectionOverrideType;
}

FBox UClothMeshSelectionTool::GetWorldSpaceFocusBox()
{
	static constexpr bool bWorld = true;
	return FBox(SelectionMechanic->GetSelectionBounds(bWorld));
}

void UClothMeshSelectionTool::SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject)
{
	DataflowContextObject = InDataflowContextObject;
}

bool UClothMeshSelectionTool::GetSelectedNodeInfo(FString& OutSelectionName, UE::Geometry::FGroupTopologySelection& OutSelection, EChaosClothAssetSelectionOverrideType& OutOverrideType)
{
	using namespace UE::Chaos::ClothAsset;

	SelectionNodeToUpdate = DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetSelectionNode_v2>();
	check(SelectionNodeToUpdate);

	// We need to sanitize the incoming indices, as the user can manually set them to anything on the node

	auto AppendVerticesIfValid = [this]<typename T>(TSet<int32>& Dest, const T& Source)
	{
		PreviewMesh->ProcessMesh([this, &Dest, &Source](const UE::Geometry::FDynamicMesh3& Mesh)
		{
			for (const int32& VertexIndex : Source)
			{
				if (Mesh.IsVertex(VertexIndex))
				{
					Dest.Add(VertexIndex);
				}
			}
		});
	};

	auto AppendFacesIfValid = [this](TSet<int32>& Dest, const TSet<int32>& Source)
	{
		PreviewMesh->ProcessMesh([this, &Dest, &Source](const UE::Geometry::FDynamicMesh3& Mesh)
		{
			for (const int32& FaceIndex : Source)
			{
				if (Mesh.IsTriangle(FaceIndex))
				{
					Dest.Add(FaceIndex);
				}
			}
		});
	};

	auto AppendSet = [this, &AppendVerticesIfValid, &AppendFacesIfValid, &OutSelection](const FChaosClothAssetNodeSelectionGroup& SourceGroup, const TSet<int32>& SourceIndices)
	{
		if (SourceGroup.Name == ClothCollectionGroup::SimVertices2D.ToString() ||
			SourceGroup.Name == ClothCollectionGroup::SimVertices3D.ToString() ||
			SourceGroup.Name == ClothCollectionGroup::RenderVertices.ToString())
		{
			if (bHasNonManifoldMapping)
			{
				for (const int32 SelectionIndex : SourceIndices)
				{
					if (SelectionIndex < SelectionToDynamicMesh.Num())		// Could be loading a render mesh selection where NumRenderVertices > NumSimVertices
					{
						AppendVerticesIfValid(OutSelection.SelectedCornerIDs, SelectionToDynamicMesh[SelectionIndex]);
					}
				}
			}
			else
			{
				AppendVerticesIfValid(OutSelection.SelectedCornerIDs, SourceIndices);
			}
		}
		else if (SourceGroup.Name == ClothCollectionGroup::SimFaces.ToString() ||
			     SourceGroup.Name == ClothCollectionGroup::RenderFaces)
		{
			AppendFacesIfValid(OutSelection.SelectedGroupIDs, SourceIndices);
		}
	};

	// Get the input set.
	InputSelectionSet.Reset();
	if (DataflowContextObject)
	{
		ensure(DataflowContextObject->IsUsingInputCollection());
		if (const TSharedPtr<const FManagedArrayCollection> ClothCollection = DataflowContextObject->GetSelectedCollection())
		{
			if (const TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = DataflowContextObject->GetDataflowContext())
			{
				using namespace UE::Chaos::ClothAsset;
				const FName InputName = SelectionNodeToUpdate->GetInputName(*DataflowContext);
				const FName GroupName = FName(*SelectionNodeToUpdate->Group.Name);

				FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection.ToSharedRef(), InputName, GroupName, InputSelectionSet);
			}
		}
	}

	TSet<int32> FinalSet;
	SelectionNodeToUpdate->CalculateFinalSet(InputSelectionSet, FinalSet);
	AppendSet(SelectionNodeToUpdate->Group, FinalSet);

	OutSelectionName = SelectionNodeToUpdate->OutputName.StringValue;
	OutOverrideType = SelectionNodeToUpdate->SelectionOverrideType;

	return true;
}


void UClothMeshSelectionTool::UpdateSelectedNode()
{
	using namespace UE::Chaos::ClothAsset;

	checkf(SelectionNodeToUpdate, TEXT("Expected non-null pointer to Selection Node"));

	// Save previous state for undo
	if (UDataflow* const Dataflow = DataflowContextObject->GetDataflowAsset())
	{
		GetToolManager()->GetContextTransactionsAPI()->AppendChange(Dataflow, 
			FChaosClothAssetSelectionNode_v2::MakeSelectedNodeChange(*SelectionNodeToUpdate),
			LOCTEXT("SelectionNodeChangeDescription", "Update Selection Node"));

		Dataflow->MarkPackageDirty();
	}

	const UE::Geometry::FGroupTopologySelection& Selection = SelectionMechanic->GetActiveSelection();

	const EClothPatternVertexType ViewMode = DataflowViewModeToClothViewMode(DataflowContextObject->GetConstructionViewMode());

	TSet<int32> Indices;
	FName GroupName = ClothCollectionGroup::SimVertices2D;

	if (SelectionMechanic->Properties->bSelectVertices)
	{
		check(!SelectionMechanic->Properties->bSelectEdges);
		check(!SelectionMechanic->Properties->bSelectFaces);

		Indices = Selection.SelectedCornerIDs;

		switch(ViewMode)
		{
		case EClothPatternVertexType::Sim2D:
			GroupName = ClothCollectionGroup::SimVertices2D;
			break;
		case EClothPatternVertexType::Sim3D:
			GroupName = ClothCollectionGroup::SimVertices3D;
			break;
		case EClothPatternVertexType::Render:
			GroupName = ClothCollectionGroup::RenderVertices;
			break;
		}
	}
	else
	{
		Indices = Selection.SelectedGroupIDs;

		switch (ViewMode)
		{
		case EClothPatternVertexType::Sim2D:
		case EClothPatternVertexType::Sim3D:
			GroupName = ClothCollectionGroup::SimFaces;
			break;
		case EClothPatternVertexType::Render:
			GroupName = ClothCollectionGroup::RenderFaces;
			break;
		}
	}

	check(SelectionNodeToUpdate);

	auto GetFinalSet = [this, &Indices, &GroupName](FChaosClothAssetNodeSelectionGroup& TargetGroup, TSet<int32>& TargetIndices)
	{
		TargetGroup.Name = GroupName.ToString();

		if (SelectionMechanic->Properties->bSelectVertices && bHasNonManifoldMapping)
		{
			TargetIndices.Reset();
			for (const int32 DynamicMeshIdx : Indices)
			{
				const int32 MappedSelectionIndex = DynamicMeshToSelection[DynamicMeshIdx];
				TargetIndices.Add(MappedSelectionIndex);
			}
		}
		else
		{
			TargetIndices = MoveTemp(Indices);
		}
	};

	SelectionNodeToUpdate->OutputName.StringValue = ToolProperties->Name;
	SelectionNodeToUpdate->SelectionOverrideType = ToolProperties->SelectionOverrideType;

	TSet<int32> FinalSet;
	GetFinalSet(SelectionNodeToUpdate->Group, FinalSet);
	SelectionNodeToUpdate->SetIndices(InputSelectionSet, FinalSet);
	
}

void UClothMeshSelectionTool::RequestAction(EClothMeshSelectionToolActions ActionType)
{
	if (!bHavePendingAction)
	{
		PendingAction = ActionType;
		bHavePendingAction = true;
	}
}


void UClothMeshSelectionTool::ApplyAction(EClothMeshSelectionToolActions ActionType)
{
	// We use a triangle topology, so the below actions can be done in mesh space instead
	//  of working with the topology.
	bool bAsTriangleTopology = true;

	switch (ActionType)
	{
	case EClothMeshSelectionToolActions::GrowSelection:
		SelectionMechanic->GrowSelection(bAsTriangleTopology);
		break;
	case EClothMeshSelectionToolActions::ShrinkSelection:
		SelectionMechanic->ShrinkSelection(bAsTriangleTopology);
		break;
	case EClothMeshSelectionToolActions::FloodSelection:
		SelectionMechanic->FloodSelection();
		break;
	case EClothMeshSelectionToolActions::ClearSelection:
		SelectionMechanic->ClearSelection();
		break;
	}
}

void UClothMeshSelectionTool::NotifyTargetChanged()
{
	using namespace UE::Chaos::ClothAsset;

	//
	// The target mesh has changed due to a view mode change. We will transfer the current in-progress selection to the new mesh.
	// First, temporarily save the existing selection
	// 

	const FGroupTopologySelection CurrentSelection = SelectionMechanic->GetActiveSelection();

	//
	// Now re-initialize everything that depends on the mesh
	//

	InitializeSculptMeshFromTarget();

	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		Topology = MakeUnique<UE::Geometry::FTriangleGroupTopology>(&Mesh, true);

		SelectionMechanic->Initialize(&Mesh,
			FTransform3d(),
			GetTargetWorld(),
			Topology.Get(),
			[this]() { return PreviewMesh->GetSpatial(); });
	});

	PreviewMesh->EnableSecondaryTriangleBuffers([this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
	});

	SelectionMechanic->NotifyMeshChanged(true);

	//
	// Copy saved values back to the new preview mesh
	//

	constexpr bool bBroadcastChange = false;
	SelectionMechanic->SetSelection(CurrentSelection, bBroadcastChange);
	GetToolManager()->PostInvalidation();
}

#undef LOCTEXT_NAMESPACE
