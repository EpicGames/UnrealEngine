// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ClothMeshSelectionTool.generated.h"

#define UE_API CHAOSCLOTHASSETEDITORTOOLS_API

class UPolygonSelectionMechanic;
class UDataflowContextObject;
class UPreviewMesh;
struct FChaosClothAssetSelectionNode_v2;
enum class EChaosClothAssetSelectionOverrideType : uint8;

namespace UE::Geometry
{
	class FGroupTopology;
	struct FGroupTopologySelection;
}

UENUM()
enum class EClothMeshSelectionToolActions
{
	NoAction,

	GrowSelection,
	ShrinkSelection,
	FloodSelection,
	ClearSelection
};


UCLASS()
class UClothMeshSelectionMechanic : public UPolygonSelectionMechanic
{
	GENERATED_BODY()

private:

	virtual bool UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut) override;
};


UCLASS(MinimalAPI)
class UClothMeshSelectionToolActions :  public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	TWeakObjectPtr<UClothMeshSelectionTool> ParentTool;

	void Initialize(UClothMeshSelectionTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EClothMeshSelectionToolActions Action);

	UFUNCTION(CallInEditor, Category = Selection)
	void GrowSelection()
	{
		PostAction(EClothMeshSelectionToolActions::GrowSelection);
	}

	UFUNCTION(CallInEditor, Category = Selection)
	void ShrinkSelection()
	{
		PostAction(EClothMeshSelectionToolActions::ShrinkSelection);
	}

	UFUNCTION(CallInEditor, Category = Selection)
	void FloodSelection()
	{
		PostAction(EClothMeshSelectionToolActions::FloodSelection);
	}

	UFUNCTION(CallInEditor, Category = Selection)
	void ClearSelection()
	{
		PostAction(EClothMeshSelectionToolActions::ClearSelection);
	}

};

UCLASS(MinimalAPI)
class UClothMeshSelectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Transient, Category = Selection, meta = (DisplayName = "Name", TransientToolProperty))
	FString Name;

	UPROPERTY(EditAnywhere, Transient, Category = Selection, meta = (TransientToolProperty))
	EChaosClothAssetSelectionOverrideType SelectionOverrideType;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Show Vertices"))
	bool bShowVertices = false;
	
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (DisplayName = "Show Edges"))
	bool bShowEdges = false;

private:

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

};

UCLASS(MinimalAPI)
class UClothMeshSelectionTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

private:

	friend class UClothMeshSelectionToolBuilder;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	UE_API void SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject);

	UE_API bool GetSelectedNodeInfo(FString& OutMapName, UE::Geometry::FGroupTopologySelection& OutSelection, EChaosClothAssetSelectionOverrideType& OutOverrideType);
	UE_API void UpdateSelectedNode();

	UPROPERTY()
	TObjectPtr<UClothMeshSelectionToolProperties> ToolProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UDataflowContextObject> DataflowContextObject = nullptr;

	TUniquePtr<UE::Geometry::FGroupTopology> Topology;

	bool bAnyChangeMade = false;

	bool bHasNonManifoldMapping = false;
	TArray<int32> DynamicMeshToSelection;
	TArray<TArray<int32>> SelectionToDynamicMesh;

	FChaosClothAssetSelectionNode_v2* SelectionNodeToUpdate = nullptr;
	TSet<int32> InputSelectionSet;

public:

	UE_API virtual void RequestAction(EClothMeshSelectionToolActions ActionType);

	UE_API void NotifyTargetChanged();

	UPROPERTY()
	TObjectPtr<UClothMeshSelectionToolActions> ActionsProps;

private:
	bool bHavePendingAction = false;
	EClothMeshSelectionToolActions PendingAction;
	UE_API virtual void ApplyAction(EClothMeshSelectionToolActions ActionType);

	UE_API void InitializeSculptMeshFromTarget();
};

#undef UE_API
