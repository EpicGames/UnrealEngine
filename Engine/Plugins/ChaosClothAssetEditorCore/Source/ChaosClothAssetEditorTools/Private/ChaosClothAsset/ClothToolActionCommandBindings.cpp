// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothToolActionCommandBindings.h"

#define LOCTEXT_NAMESPACE "ClothToolActionCommandBindings"

FClothEditorWeightMapPaintToolActionCommands::FClothEditorWeightMapPaintToolActionCommands() : FClothToolActionCommands<FClothEditorWeightMapPaintToolActionCommands, UClothEditorWeightMapPaintTool>(
	TEXT("ClothEditorWeightMapPaintToolContext"), LOCTEXT("ClothEditorWeightMapPaintToolContext", "Cloth Weight Map Paint Tool Context"))
{}

FClothMeshSelectionToolActionCommands::FClothMeshSelectionToolActionCommands() : FClothToolActionCommands<FClothMeshSelectionToolActionCommands, UClothMeshSelectionTool>(
	TEXT("ClothSelectionToolContext"), LOCTEXT("ClothSelectionToolContext", "Cloth Selection Tool Context"))
{}

FClothTransferSkinWeightsToolActionCommands::FClothTransferSkinWeightsToolActionCommands() : FClothToolActionCommands<FClothTransferSkinWeightsToolActionCommands, UClothTransferSkinWeightsTool>(
	TEXT("ClothTransferSkinWeightsToolContext"), LOCTEXT("ClothTransferSkinWeightsToolContext", "Cloth Transfer Skin Weights Tool Context"))
{}

FClothToolActionCommandBindings::FClothToolActionCommandBindings()
{
	// Note: if a TCommands<> doesn't actually register any commands then it will be deleted. Only WeightMapPaintTool currently has key commands, but we will include the other tools
	// here so that hotkeys can be added to them in the future. This means we need to check if the objects are registered before trying to use them below.

	FClothEditorWeightMapPaintToolActionCommands::Register();
	FClothMeshSelectionToolActionCommands::Register();
	FClothTransferSkinWeightsToolActionCommands::Register();
}

void FClothToolActionCommandBindings::UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const
{
	if (FClothEditorWeightMapPaintToolActionCommands::IsRegistered())
	{
		FClothEditorWeightMapPaintToolActionCommands::Get().UnbindActiveCommands(UICommandList);
	}

	if (FClothMeshSelectionToolActionCommands::IsRegistered())
	{
		FClothMeshSelectionToolActionCommands::Get().UnbindActiveCommands(UICommandList);
	}

	if (FClothTransferSkinWeightsToolActionCommands::IsRegistered())
	{
		FClothTransferSkinWeightsToolActionCommands::Get().UnbindActiveCommands(UICommandList);
	}
}

void FClothToolActionCommandBindings::BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const
{
	if (ExactCast<UClothEditorWeightMapPaintTool>(Tool) && FClothEditorWeightMapPaintToolActionCommands::IsRegistered())
	{
		FClothEditorWeightMapPaintToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
	}
	else if (ExactCast<UClothMeshSelectionTool>(Tool) && FClothMeshSelectionToolActionCommands::IsRegistered())
	{
		FClothMeshSelectionToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
	}
	else if (ExactCast<UClothTransferSkinWeightsTool>(Tool) && FClothTransferSkinWeightsToolActionCommands::IsRegistered())
	{
		FClothTransferSkinWeightsToolActionCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
	}
}

#undef LOCTEXT_NAMESPACE

