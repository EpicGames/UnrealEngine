// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowToolRegistry.h"
#include "Tools/InteractiveToolsCommands.h"
#include "Styling/AppStyle.h"			// Style name is a required parameter of TInteractiveToolCommands/FBindingContext

// For GetToolDefaultObjectList in class template 
#include "ClothWeightMapPaintTool.h"	
#include "ClothMeshSelectionTool.h"
#include "ClothTransferSkinWeightsTool.h"


// TInteractiveToolCommands<> are typically used to bind the current set of available hotkey commands when a tool starts/ends. However we cannot store multiple actions 
// with the same key activation in a single TInteractiveToolCommands object (even if they are only active in different tools), so we end up creating one TInteractiveToolCommands per tool. 
// This is also what Modeling Tools does -- see comments in ModelingToolsActions.h.


// Base class with TInteractiveToolCommands<> boilerplate

template<typename T, typename ToolClass>
class FClothToolActionCommands : public TInteractiveToolCommands<T>
{
public:
	FClothToolActionCommands(const FName InContextName, const FText& InContextDesc) : TInteractiveToolCommands<T>(InContextName, InContextDesc, NAME_None, FAppStyle::GetAppStyleSetName())
	{}

	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override
	{
		ToolCDOs.Add(GetMutableDefault<ToolClass>());
	}
};

// Derived classes for individual tools

class FClothEditorWeightMapPaintToolActionCommands : public FClothToolActionCommands<FClothEditorWeightMapPaintToolActionCommands, UClothEditorWeightMapPaintTool>
{
public:
	FClothEditorWeightMapPaintToolActionCommands();
};

class FClothMeshSelectionToolActionCommands : public FClothToolActionCommands<FClothMeshSelectionToolActionCommands, UClothMeshSelectionTool>
{
public:
	FClothMeshSelectionToolActionCommands();
};

class FClothTransferSkinWeightsToolActionCommands : public FClothToolActionCommands<FClothTransferSkinWeightsToolActionCommands, UClothTransferSkinWeightsTool>
{
public:
	FClothTransferSkinWeightsToolActionCommands();
};


// Tool action registry entry

class FClothToolActionCommandBindings : public UE::Dataflow::FDataflowToolRegistry::IDataflowToolActionCommands
{

public:
	FClothToolActionCommandBindings();
	virtual void UnbindActiveCommands(const TSharedPtr<FUICommandList>& UICommandList) const override;
	virtual void BindCommandsForCurrentTool(const TSharedPtr<FUICommandList>& UICommandList, UInteractiveTool* Tool) const override;
};
