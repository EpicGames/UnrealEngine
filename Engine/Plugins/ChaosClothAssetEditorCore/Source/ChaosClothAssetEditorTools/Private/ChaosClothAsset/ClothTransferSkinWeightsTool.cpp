// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothTransferSkinWeightsTool.h"

#include "SkeletalMeshAttributes.h"
#include "Engine/World.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Engine/SkeletalMesh.h"
#include "ContextObjectStore.h"
#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "InteractiveToolObjects.h"
#include "MeshOpPreviewHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothTransferSkinWeightsTool)

#define LOCTEXT_NAMESPACE "ClothTransferSkinWeightsTool"

namespace UE::Chaos::ClothAsset::Private
{
	/** An empty operator in case we need to add a background compute operation in the future. */
	class FClothTransferSkinWeightsOp : public UE::Geometry::FDynamicMeshOperator
	{
	public:
		FClothTransferSkinWeightsOp()
		{
		}

	private:
		// FDynamicMeshOperator interface
		virtual void CalculateResult(FProgressCancel* Progress) override
		{
			using namespace UE::Geometry;

			FGeometryResult OpResult;
			OpResult.Result = EGeometryResultType::Success;
			SetResultInfo(OpResult);
		}
	};
}


// ------------------- Tool -------------------


void UClothTransferSkinWeightsTool::Setup()
{
	USingleSelectionMeshEditingTool::Setup();

	TransferSkinWeightsNode = DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetTransferSkinWeightsNode>();
	checkf(TransferSkinWeightsNode, TEXT("No Transfer Skin Weights Node is currently selected, or more than one node is selected"));

	ToolProperties = NewObject<UClothTransferSkinWeightsToolProperties>(this);

	SetSRTPropertiesFromTransform(TransferSkinWeightsNode->Transform);
	ToolProperties->SourceMesh = TransferSkinWeightsNode->SkeletalMesh;

	AddToolPropertySource(ToolProperties);


	//
	// Set up Preview mesh that will show the results of the computation
	//

	TargetClothPreview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	TargetClothPreview->Setup(GetTargetWorld(), this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(TargetClothPreview->PreviewMesh, Target);
	UMaterialInterface* const TargetMaterial = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
	TargetClothPreview->ConfigureMaterials(TargetMaterial, TargetMaterial);

	// Mesh topology is not being changed 
	TargetClothPreview->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::VertexColors);

	TargetClothPreview->OnOpCompleted.AddUObject(this, &UClothTransferSkinWeightsTool::OpFinishedCallback);
	TargetClothPreview->OnMeshUpdated.AddUObject(this, &UClothTransferSkinWeightsTool::PreviewMeshUpdatedCallback);

	// Set the initial preview mesh before any computation runs
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	UE::Geometry::FDynamicMesh3 InitialPreviewMesh = UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams);
	TargetClothPreview->PreviewMesh->UpdatePreview(MoveTemp(InitialPreviewMesh));

	TargetClothPreview->SetVisibility(true);

	//
	// Source mesh (populated from the SkeletalMesh tool property)
	//

	SourceMeshParentActor = GetTargetWorld()->SpawnActor<AInternalToolFrameworkActor>();
	SourceMeshComponent = NewObject<USkeletalMeshComponent>(SourceMeshParentActor);
	SourceMeshComponent->SetDisablePostProcessBlueprint(true); 
	SourceMeshParentActor->SetRootComponent(SourceMeshComponent);
	SourceMeshComponent->RegisterComponent();

	// Watch for property changes
	ToolProperties->WatchProperty(TransferSkinWeightsNode->Transform, 
	[this](const FTransform& Transform) 
	{
		SetSRTPropertiesFromTransform(Transform);
	}, 
	[this](const FTransform& Transform1, const FTransform& Transform2)
	{ 
		return !Transform1.Equals(Transform2, 0.f);
	});

	ToolProperties->WatchProperty(ToolProperties->SourceMesh, [this](TObjectPtr<USkeletalMesh> Mesh) { UpdateSourceMesh(Mesh); });

	ToolProperties->WatchProperty(TransferSkinWeightsNode->SkeletalMesh, [this](TObjectPtr<USkeletalMesh> Mesh) 
	{ 
		ToolProperties->SourceMesh = Mesh; // this triggers the WatchProperty of ToolProperties->SourceMesh above
	});
	
	ToolProperties->WatchProperty(ToolProperties->bHideSourceMesh, [this](bool bNewHideSourceMesh) 
	{ 
		check(SourceMeshComponent);
		SourceMeshComponent->SetVisibility(!bNewHideSourceMesh);
	});

	//
	// Transform/Gizmo/Proxy stuff
	//

	ToolProperties->WatchProperty(ToolProperties->SourceMeshTranslation, [this](const FVector3d& NewTranslation)
	{
		if (DataBinder)
		{
			DataBinder->UpdateAfterDataEdit();
		}
	});

	ToolProperties->WatchProperty(ToolProperties->SourceMeshRotation, [this](const FVector3d& NewTranslation)
	{
		if (DataBinder)
		{
			DataBinder->UpdateAfterDataEdit();
		}
	});

	ToolProperties->WatchProperty(ToolProperties->SourceMeshScale, [this](const FVector3d& NewTranslation)
	{
		if (DataBinder)
		{
			DataBinder->UpdateAfterDataEdit();
		}
	});


	UInteractiveGizmoManager* const GizmoManager = GetToolManager()->GetPairedGizmoManager();
	ensure(GizmoManager);
	SourceMeshTransformProxy = NewObject<UTransformProxy>(this);
	ensure(SourceMeshTransformProxy);
	SourceMeshTransformProxy->SetTransform(TransferSkinWeightsNode->Transform);

	SourceMeshTransformProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform NewTransform)
	{
		if (SourceMeshParentActor)
		{
			SourceMeshParentActor->SetActorTransform(NewTransform);
		}
	});

	SourceMeshTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, ETransformGizmoSubElements::FullTranslateRotateScale, this);
	ensure(SourceMeshTransformGizmo);

	SourceMeshTransformGizmo->SetActiveTarget(SourceMeshTransformProxy, GetToolManager());
	SourceMeshTransformGizmo->SetVisibility(ToolProperties->SourceMesh != nullptr);
	SourceMeshTransformGizmo->bUseContextCoordinateSystem = false;
	SourceMeshTransformGizmo->bUseContextGizmoMode = false;
	SourceMeshTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	DataBinder = MakeShared<FTransformGizmoDataBinder>();
	DataBinder->InitializeBoundVectors(&ToolProperties->SourceMeshTranslation, &ToolProperties->SourceMeshRotation, &ToolProperties->SourceMeshScale);
	DataBinder->BindToInitializedGizmo(SourceMeshTransformGizmo, SourceMeshTransformProxy);


	UpdateSourceMesh(ToolProperties->SourceMesh);

	UE::ToolTarget::HideSourceObject(Target);
}

void UClothTransferSkinWeightsTool::Shutdown(EToolShutdownType ShutdownType)
{
	USingleSelectionMeshEditingTool::Shutdown(ShutdownType);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		TransferSkinWeightsNode->SkeletalMesh = ToolProperties->SourceMesh;
		TransferSkinWeightsNode->Transform = TransformFromProperties();
		TransferSkinWeightsNode->Invalidate();
		if (DataflowContextObject)
		{
			if (TObjectPtr<UDataflow> const Dataflow = DataflowContextObject->GetDataflowAsset())
			{
				Dataflow->MarkPackageDirty();
			}
		}
	}

	if (SourceMeshTransformProxy)
	{
		SourceMeshTransformProxy->OnTransformChanged.RemoveAll(this);
		SourceMeshTransformProxy->OnEndTransformEdit.RemoveAll(this);
	}

	if (TargetClothPreview)
	{
		TargetClothPreview->OnMeshUpdated.RemoveAll(this);
		TargetClothPreview->Shutdown();
		TargetClothPreview = nullptr;
	}

	if (SourceMeshComponent)
	{
		SourceMeshComponent->DestroyComponent();
		SourceMeshComponent = nullptr;
	}

	if (SourceMeshParentActor)
	{
		SourceMeshParentActor->Destroy();
		SourceMeshParentActor = nullptr;
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	SourceMeshTransformGizmo = nullptr;

	UE::ToolTarget::ShowSourceObject(Target);
}


bool UClothTransferSkinWeightsTool::CanAccept() const
{
	const FTransform& TransformOnNode = TransferSkinWeightsNode->Transform;;

	return (ToolProperties->SourceMesh != TransferSkinWeightsNode->SkeletalMesh) ||
		(ToolProperties->SourceMeshRotation != TransformOnNode.Rotator().Euler()) ||
		(ToolProperties->SourceMeshTranslation != TransformOnNode.GetTranslation()) ||
		(ToolProperties->SourceMeshScale != TransformOnNode.GetScale3D());

}

void UClothTransferSkinWeightsTool::OnTick(float DeltaTime)
{
	if (TargetClothPreview)
	{
		TargetClothPreview->Tick(DeltaTime);
	}
}

TUniquePtr<UE::Geometry::FDynamicMeshOperator> UClothTransferSkinWeightsTool::MakeNewOperator()
{
	TUniquePtr<UE::Chaos::ClothAsset::Private::FClothTransferSkinWeightsOp> TransferOp =
		MakeUnique<UE::Chaos::ClothAsset::Private::FClothTransferSkinWeightsOp>();
	return TransferOp;
}

void UClothTransferSkinWeightsTool::SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject)
{
	DataflowContextObject = InDataflowContextObject;
}

FTransform UClothTransferSkinWeightsTool::TransformFromProperties() const
{
	const FRotator Rotation = FRotator::MakeFromEuler(ToolProperties->SourceMeshRotation);
	return FTransform(Rotation, ToolProperties->SourceMeshTranslation, ToolProperties->SourceMeshScale);
}

void UClothTransferSkinWeightsTool::SetSRTPropertiesFromTransform(const FTransform& Transform) const
{
	ToolProperties->SourceMeshRotation = Transform.Rotator().Euler();
	ToolProperties->SourceMeshTranslation = Transform.GetTranslation();
	ToolProperties->SourceMeshScale = Transform.GetScale3D();
}

void UClothTransferSkinWeightsTool::UpdateSourceMesh(TObjectPtr<USkeletalMesh> Mesh)
{
	checkf(ToolProperties, TEXT("ToolProperties is expected to be non-null. Be sure to run Setup() on this tool when it is created."));

	if (Mesh == SourceMeshComponent->GetSkeletalMeshAsset())
	{
		return;
	}

	SourceMeshComponent->SetSkeletalMeshAsset(Mesh);

	if (Mesh)
	{
		// Set up source mesh (from the SkeletalMesh)
		SourceMeshParentActor->SetActorTransform(TransformFromProperties());
		SourceMeshComponent->SetVisibility(!ToolProperties->bHideSourceMesh);

		// Use ReinitializeGizmoTransform rather than SetNewGizmoTransform to avoid having this on the undo stack
		SourceMeshTransformGizmo->ReinitializeGizmoTransform(SourceMeshParentActor->GetActorTransform());
		SourceMeshTransformGizmo->SetVisibility(!ToolProperties->bHideSourceMesh);
		SourceMeshTransformGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
	}
	else
	{
		SourceMeshComponent->SetVisibility(false);
		SourceMeshTransformGizmo->SetVisibility(false);
	}	
}


void UClothTransferSkinWeightsTool::OpFinishedCallback(const UE::Geometry::FDynamicMeshOperator* Op)
{
	if (Op->GetResultInfo().Result == UE::Geometry::EGeometryResultType::Failure)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("TransferOpFailedWarning", "Operation failed"), EToolMessageLevel::UserWarning);
		bHasOpFailedWarning = true;
	}
	else
	{
		if (bHasOpFailedWarning)
		{
			GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);    // clear old warning
			bHasOpFailedWarning = false;
		}
	}
}


void UClothTransferSkinWeightsTool::PreviewMeshUpdatedCallback(UMeshOpPreviewWithBackgroundCompute* Preview)
{
}

#undef LOCTEXT_NAMESPACE
