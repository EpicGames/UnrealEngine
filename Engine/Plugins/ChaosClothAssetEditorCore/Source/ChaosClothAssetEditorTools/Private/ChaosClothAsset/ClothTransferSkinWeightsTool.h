// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "ModelingOperators.h"
#include "Transforms/TransformGizmoDataBinder.h"
#include "ClothTransferSkinWeightsTool.generated.h"

#define UE_API CHAOSCLOTHASSETEDITORTOOLS_API


class UClothTransferSkinWeightsTool;
class USkeletalMesh;
class UDataflowContextObject;
class UTransformProxy;
class UCombinedTransformGizmo;
class UMeshOpPreviewWithBackgroundCompute;
class AInternalToolFrameworkActor;
class USkeletalMeshComponent;
class FTransformGizmoDataBinder;
struct FChaosClothAssetTransferSkinWeightsNode;

UCLASS(MinimalAPI)
class UClothTransferSkinWeightsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = Source)
	TObjectPtr<USkeletalMesh> SourceMesh;

	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DisplayName = "Location", EditCondition = "SourceMesh != nullptr"))
	FVector3d SourceMeshTranslation = FVector3d::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DisplayName = "Rotation", EditCondition = "SourceMesh != nullptr"))
	FVector3d SourceMeshRotation = FVector3d::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DisplayName = "Scale", AllowPreserveRatio, EditCondition = "SourceMesh != nullptr"))
	FVector3d SourceMeshScale = FVector3d::OneVector;

	UPROPERTY(EditAnywhere, Category = Source)
	bool bHideSourceMesh = false;
};

UCLASS(MinimalAPI)
class UClothTransferSkinWeightsTool : public USingleSelectionMeshEditingTool, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

private:

	friend class UClothTransferSkinWeightsToolBuilder;

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual bool HasAccept() const override { return true; }
	virtual bool HasCancel() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual void OnTick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;
	
	UE_API void SetDataflowEditorContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject);

	UE_API FTransform TransformFromProperties() const;
	UE_API void SetSRTPropertiesFromTransform(const FTransform& Transform) const;

	UE_API void UpdateSourceMesh(TObjectPtr<USkeletalMesh> Mesh);

	UE_API void OpFinishedCallback(const UE::Geometry::FDynamicMeshOperator* Op);

	UE_API void PreviewMeshUpdatedCallback(UMeshOpPreviewWithBackgroundCompute* Preview);


	UPROPERTY(Transient)
	TObjectPtr<UClothTransferSkinWeightsToolProperties> ToolProperties;

	UPROPERTY(Transient)
	TObjectPtr<UDataflowContextObject> DataflowContextObject = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> TargetClothPreview;

	UPROPERTY(Transient)
	TObjectPtr<AInternalToolFrameworkActor> SourceMeshParentActor;

	UPROPERTY(Transient)
	TObjectPtr<USkeletalMeshComponent> SourceMeshComponent;

	// Source mesh transform gizmo support
	UPROPERTY(Transient)
	TObjectPtr<UTransformProxy> SourceMeshTransformProxy;

	UPROPERTY(Transient)
	TObjectPtr<UCombinedTransformGizmo> SourceMeshTransformGizmo;

	TSharedPtr<FTransformGizmoDataBinder> DataBinder;

	FChaosClothAssetTransferSkinWeightsNode* TransferSkinWeightsNode = nullptr;

	bool bHasOpFailedWarning = false;
};


#undef UE_API
