// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothWeightMapPaintBrushOps.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"

//
// Paint Brush
//

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothWeightMapPaintBrushOps)

void FWeightMapPaintBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& NewAttributesOut)
{
	UWeightMapPaintBrushOpProps* const Props = GetPropertySetAs<UWeightMapPaintBrushOpProps>();
	const double TargetValue = (double)Props->GetAttribute();

	const int32 NumVertices = Vertices.Num();
	ParallelFor(NumVertices, [&](int32 k)
	{
		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const int32 VertIdx = Vertices[k];
			const FVector3d VertexPos = Mesh->GetVertex(VertIdx);
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				return;
			}
		}

		const double ExistingValue = NewAttributesOut[k];
		const double ValueDiff = TargetValue - ExistingValue;
		NewAttributesOut[k] = FMath::Clamp(ExistingValue + Stamp.Power * ValueDiff, 0.0, 1.0);
	});
}


//
// Erase Brush
//

void FWeightMapEraseBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& NewAttributesOut)
{
	UWeightMapEraseBrushOpProps* Props = GetPropertySetAs<UWeightMapEraseBrushOpProps>();
	const double EraseAttribute = (double)Props->GetAttribute();

	// TODO: Add something here to get the old value so we can subtract (clamped) the AttributeValue from it.
	// TODO: Handle the stamp's properties for fall off, etc..

	check(NewAttributesOut.Num() == Vertices.Num());

	for (int32 k = 0; k < Vertices.Num(); ++k)
	{
		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const int32 VertIdx = Vertices[k];
			const FVector3d VertexPos = Mesh->GetVertex(VertIdx);
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				continue;
			}
		}
		NewAttributesOut[k] = EraseAttribute;
	}
}


//
// Smooth Brush
//

void FWeightMapSmoothBrushOp::ApplyStampByVertices(
	const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp,
	const TArray<int32>& Vertices,
	TArray<double>& VertexWeightValues)
{
	// Converted from FClothPaintTool_Smooth::SmoothVertices
	
	const int32 NumVertices = Vertices.Num();

	TMap<int32, int32> VertexToBufferIndexMap;
	for (int32 BufferIndex = 0; BufferIndex < NumVertices; ++BufferIndex)
	{
		const int32 VertexIndex = Vertices[BufferIndex];
		VertexToBufferIndexMap.Add(VertexIndex, BufferIndex);
	}

	// Compute average values of one-rings for all vertices
	TArray<double> OneRingAverages;
	OneRingAverages.SetNumUninitialized(NumVertices);

	for (int32 BufferIndex = 0; BufferIndex < NumVertices; ++BufferIndex)
	{
		const int32 VertexIndex = Vertices[BufferIndex];
		double Accumulator = 0.0f;
		int NumNeighbors = 0;

		for (const int32 NeighborIndex : Mesh->VtxVerticesItr(VertexIndex))
		{
			if (VertexToBufferIndexMap.Contains(NeighborIndex))
			{
				const int32 NeighborBufferIndex = VertexToBufferIndexMap[NeighborIndex];
				Accumulator += VertexWeightValues[NeighborBufferIndex];
				++NumNeighbors;
			}
		}

		if (NumNeighbors > 0)
		{
			OneRingAverages[BufferIndex] = Accumulator / NumNeighbors;
		}
		else
		{
			// Don't change the vertex value if it has no neighbors
			OneRingAverages[BufferIndex] = VertexWeightValues[BufferIndex];
		}
	}

	// Blend vertex value with its average one-ring value
	for (int32 BufferIndex = 0; BufferIndex < NumVertices; ++BufferIndex)
	{
		const double Diff = OneRingAverages[BufferIndex] - VertexWeightValues[BufferIndex];
		
		const int32 VertexIndex = Vertices[BufferIndex];
		const FVector3d VertexPos = Mesh->GetVertex(VertexIndex);

		if (bApplyRadiusLimit)
		{
			const FVector3d& StampPos = Stamp.LocalFrame.Origin;
			const double DistanceSquared = (VertexPos - StampPos).SquaredLength();
			if (DistanceSquared >= Stamp.Radius * Stamp.Radius)
			{
				continue;
			}
		}

		double FalloffScalar = 1.0;
		if (this->Falloff)
		{
			FalloffScalar = GetFalloff().Evaluate(Stamp, VertexPos);
		}

		VertexWeightValues[BufferIndex] += FalloffScalar * Stamp.Power * Diff;
	}
}

