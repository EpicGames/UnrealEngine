// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "ClothWeightMapPaintBrushOps.generated.h"




class FMeshVertexWeightMapEditBrushOp : public FMeshSculptBrushOp
{
public:

	// not supported for this kind of brush op
	virtual void ApplyStamp(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<FVector3d>& NewColorsOut) override
	{
		check(false);
	}


	virtual void ApplyStampByVertices(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<double>& NewAttributesOut) = 0;

	/** If bApplyRadiusLimit is enabled, only vertices within the stamp radius are considered */
	bool bApplyRadiusLimit = true;

};



//
// Erase Brush
// 

UCLASS(MinimalAPI)
class UWeightMapEraseBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	// TODO: This AttributeValue is not currenly connected to the UI (jira UE-198413)
	/** Attribute to set as Erased value */
	UPROPERTY(EditAnywhere, Category = EraseBrush, meta = (DisplayName = "Erase Attribute", UIMin = 0))
	double AttributeValue = 0;

	virtual double GetAttribute() { return AttributeValue; }

	virtual float GetFalloff() override { return 0.0f; }
};



class FWeightMapEraseBrushOp : public FMeshVertexWeightMapEditBrushOp
{
public:

	virtual void ApplyStampByVertices(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<double>& NewAttributesOut) override;

};


//
// Paint Brush
// 

UCLASS(MinimalAPI)
class UWeightMapPaintBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:

	/** The Attribute that will be assigned to triangles within the brush region */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Value", UIMin = 0))
	double AttributeValue = 1;

	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10."))
	float Strength = 0.5;

	virtual double GetAttribute() { return AttributeValue; }

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = NewStrength;  }

	virtual float GetFalloff() override { return 0.0f; }
};


class FWeightMapPaintBrushOp : public FMeshVertexWeightMapEditBrushOp
{
public:

	virtual bool IgnoreZeroMovements() const override
	{
		return false;
	}

	virtual void ApplyStampByVertices(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<double>& NewAttributesOut) override;

};

//
// Smooth Brush
// 

UCLASS(MinimalAPI)
class UWeightMapSmoothBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10."))
	float Strength = 0.5;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = NewStrength; }

	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.", ClampMin = "0.0", ClampMax = "1."))
	float Falloff = 0.5;

	virtual float GetFalloff() override { return Falloff; }
	virtual void SetFalloff(float NewFalloff) override { Falloff = NewFalloff; }
};


class FWeightMapSmoothBrushOp : public FMeshVertexWeightMapEditBrushOp
{
public:

	virtual bool IgnoreZeroMovements() const override
	{
		return false;
	}

	virtual void ApplyStampByVertices(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<double>& NewAttributesOut) override;

};
