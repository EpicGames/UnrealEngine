// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothWeightMapPaintTool.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Util/BufferUtil.h"
#include "Util/ColorConstants.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Polygon2.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/BasicChanges.h"

#include "ChaosClothAsset/ClothWeightMapPaintBrushOps.h"
#include "Sculpting/StampFalloffs.h"
#include "Sculpting/MeshSculptUtil.h"

#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothEditorToolBuilders.h"
#include "ChaosClothAsset/ClothPatternVertexType.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ContextObjectStore.h"
#include "Dataflow/DataflowContextObject.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowObject.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "GraphEditor.h"
#include "Dataflow/DataflowSNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "DynamicSubmesh3.h"
#include "Intersection/IntrRay3Triangle3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothWeightMapPaintTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UClothEditorWeightMapPaintTool"

namespace UE::Chaos::ClothAsset::Private
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution WeightPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}


/*
 * ToolActions
 */
void UClothEditorMeshWeightMapPaintToolActions::PostAction(EClothEditorWeightMapPaintToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


/*
 * Properties
 */
void UClothEditorUpdateWeightMapProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UClothEditorUpdateWeightMapProperties, Name))
	{
		UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Name);
	}
}


// Show/Hide properties
void UClothEditorMeshWeightMapPaintToolShowHideProperties::PostAction(EClothEditorWeightMapPaintToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


/*
 * Tool
 */

void UClothEditorWeightMapPaintTool::SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior)
{
	// map the horizontal behavior to change the brush size
	UMeshSculptToolBase::MapHorizontalBrushEditBehaviorToBrushSize(OutBehavior);

	// map the vertical ehavior to change the attribiute value
	OutBehavior.VerticalProperty.GetValueFunc = [this]()
		{
			return FilterProperties->AttributeValue;
		};

	OutBehavior.VerticalProperty.SetValueFunc = [this](float NewValue)
		{
			FilterProperties->AttributeValue = FMath::Min(1.0f, FMath::Max(NewValue, 0.f));
#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UClothEditorWeightMapPaintBrushFilterProperties, AttributeValue)));
			FilterProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		};
	OutBehavior.VerticalProperty.Name = LOCTEXT("AttributeValue", "Attribute Value");
	OutBehavior.VerticalProperty.EditRate = 0.005f;
	OutBehavior.VerticalProperty.bEnabled = true;
}


void UClothEditorWeightMapPaintTool::Setup()
{
	UMeshSculptToolBase::Setup();

	// Get the selected weight map node
	WeightMapNodeToUpdate = DataflowContextObject->GetSelectedNodeOfType<FChaosClothAssetWeightMapNode>();
	checkf(WeightMapNodeToUpdate, TEXT("No Weight Map Node is currently selected, or more than one node is selected"));

	SetToolDisplayName(LOCTEXT("ToolName", "Paint Weight Maps"));

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);

	// Set up control points mechanic
	PolyLassoMechanic = NewObject<UPolyLassoMarqueeMechanic>(this);
	PolyLassoMechanic->Setup(this);
	PolyLassoMechanic->SetIsEnabled(false);
	PolyLassoMechanic->SpacingTolerance = 10.0f;
	PolyLassoMechanic->OnDrawPolyLassoFinished.AddUObject(this, &UClothEditorWeightMapPaintTool::OnPolyLassoFinished);


	// Set up vertex selection mechanic
	PolygonSelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	PolygonSelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	PolygonSelectionMechanic->Setup(this);
	PolygonSelectionMechanic->SetIsEnabled(false);
	PolygonSelectionMechanic->OnSelectionChanged.AddUObject(this, &UClothEditorWeightMapPaintTool::OnSelectionModified);

	// disable CTRL to remove from selection
	PolygonSelectionMechanic->SetShouldRemoveFromSelectionFunc([]() { return false; });

	PolygonSelectionMechanic->Properties->bSelectEdges = false;
	PolygonSelectionMechanic->Properties->bSelectFaces = false;
	PolygonSelectionMechanic->Properties->bSelectVertices = true;

	UpdateWeightMapProperties = NewObject<UClothEditorUpdateWeightMapProperties>(this);
	UpdateWeightMapProperties->Name = WeightMapNodeToUpdate->OutputName.StringValue;
	UpdateWeightMapProperties->MapOverrideType = WeightMapNodeToUpdate->MapOverrideType;

	UpdateWeightMapProperties->WatchProperty(WeightMapNodeToUpdate->OutputName.StringValue, [this](const FString& NewName)
	{
		UpdateWeightMapProperties->Name = NewName;
	});
	AddToolPropertySource(UpdateWeightMapProperties);

	// initialize other properties
	FilterProperties = NewObject<UClothEditorWeightMapPaintBrushFilterProperties>(this);
	FilterProperties->WatchProperty(FilterProperties->ColorMap,
		[this](EClothEditorWeightMapDisplayType NewType) 
		{ 
			UpdateVertexColorOverlay(); 
			DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
		});
	FilterProperties->WatchProperty(FilterProperties->bHighlightZeroAndOne,
		[this](bool bNewValue)
		{
			UpdateVertexColorOverlay();
			DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
		});
	FilterProperties->WatchProperty(FilterProperties->SubToolType,
		[this](EClothEditorWeightMapPaintInteractionType NewType) { UpdateSubToolType(NewType); });
	FilterProperties->WatchProperty(FilterProperties->BrushSize, [this](float NewSize) 
		{ 
			UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize = NewSize; 
			CalculateBrushRadius();
		});

	FilterProperties->WatchProperty(FilterProperties->Falloff, [this](double NewFalloff)
		{
			// Brush indicator rendering uses this value
			GetActiveBrushOp()->PropertySet->SetFalloff(NewFalloff);
		});


	FilterProperties->BrushSize = UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize;
	FilterProperties->RestoreProperties(this);
	AddToolPropertySource(FilterProperties);

	InitializeIndicator();

	// initialize our properties
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = true;
	UMeshSculptToolBase::BrushProperties->bShowLazyness = false;
	UMeshSculptToolBase::BrushProperties->FlowRate = 0.0f;
	CalculateBrushRadius();

	PaintBrushOpProperties = NewObject<UWeightMapPaintBrushOpProps>(this);
	RegisterBrushType((int32)EClothEditorWeightMapPaintBrushType::Paint, LOCTEXT("Paint", "Paint"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FWeightMapPaintBrushOp>(); }),
		PaintBrushOpProperties);

	SmoothBrushOpProperties = NewObject<UWeightMapSmoothBrushOpProps>(this);
	RegisterBrushType((int32)EClothEditorWeightMapPaintBrushType::Smooth, LOCTEXT("SmoothBrushType", "Smooth"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FWeightMapSmoothBrushOp>(); }),
		SmoothBrushOpProperties);

	// secondary brushes
	EraseBrushOpProperties = NewObject<UWeightMapEraseBrushOpProps>(this);

	RegisterSecondaryBrushType((int32)EClothEditorWeightMapPaintBrushType::Erase, LOCTEXT("Erase", "Erase"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FWeightMapEraseBrushOp>>(),
		EraseBrushOpProperties);

	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::ViewProperties, true);

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);

	ActionsProps = NewObject<UClothEditorMeshWeightMapPaintToolActions>(this);
	ActionsProps->Initialize(this);
	AddToolPropertySource(ActionsProps);

	// register watchers
	FilterProperties->WatchProperty( FilterProperties->PrimaryBrushType,
		[this](EClothEditorWeightMapPaintBrushType NewType) { UpdateBrushType(NewType); });

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(DynamicMeshComponent->GetWorld(), DynamicMeshComponent->GetComponentTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowNormalSeams = false;
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("ClothEditorWeightMapPaintTool2"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) 
	{
		if (HiddenTriangles.Num() > 0 || PendingHiddenTriangles.Num() > 0)
		{
			const UE::Geometry::FDynamicMesh3* const FullMesh = GetSculptMesh();

			TArray<int> NonHiddenTriangles;
			for (const int32 TID : FullMesh->TriangleIndicesItr())
			{
				if (!HiddenTriangles.Contains(TID) && !PendingHiddenTriangles.Contains(TID))
				{
					NonHiddenTriangles.Add(TID);
				}
			}

			UE::Geometry::FDynamicSubmesh3 Submesh(FullMesh, NonHiddenTriangles);
			ProcessFunc(Submesh.GetSubmesh());
		}
		else
		{
			ProcessFunc(*GetSculptMesh());
		}
	});

	ShowHideProperties = NewObject<UClothEditorMeshWeightMapPaintToolShowHideProperties>();
	ShowHideProperties->Initialize(this);
	ShowHideProperties->WatchProperty(ShowHideProperties->ShowPatterns,
		[this](const TMap<int32, bool>& NewMap)
		{
			bool bAnySelected = false;
			for (const TPair<int32, bool>& KeyValue : NewMap)
			{
				if (KeyValue.Value)
				{
					bAnySelected = true;
					break;
				}
			}

			HiddenTriangles.Reset();
			if (bAnySelected)
			{
				for (const TPair<int32, bool>& KeyValue : NewMap)
				{
					if (KeyValue.Value == false)
					{
						const int32 PatternIndex = KeyValue.Key;
						if (PatternIndex < PatternTriangleOffsetAndNum.Num())
						{
							const TPair<int32, int32> StartAndNum = PatternTriangleOffsetAndNum[PatternIndex];
							for (int32 TID = StartAndNum.Key; TID < StartAndNum.Key + StartAndNum.Value; ++TID)
							{
								HiddenTriangles.Add(TID);
							}
						}
					}
				}
			}

			MeshElementsDisplay->NotifyMeshChanged();
			DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
		},
		[this](const TMap<int32, bool>& A, const TMap<int32, bool>& B)		// Not-equal function for TMap
		{
			return (!A.OrderIndependentCompareEqual(B));
		}
	);
	AddToolPropertySource(ShowHideProperties);

	UpdateWeightMapProperties->Name = WeightMapNodeToUpdate->OutputName.StringValue;
	UpdateWeightMapProperties->MapOverrideType = WeightMapNodeToUpdate->MapOverrideType;
	SetToolPropertySourceEnabled(UpdateWeightMapProperties, true);

	// disable view properties
	SetViewPropertiesEnabled(false);
	UpdateMaterialMode(EMeshEditingMaterialModes::VertexColor);
	UpdateWireframeVisibility(false);
	UpdateFlatShadingSetting(false);

	// configure panels
	UpdateSubToolType(FilterProperties->SubToolType);

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();

	UpdateBrushType(FilterProperties->PrimaryBrushType);
	SetActiveSecondaryBrushType((int32)EClothEditorWeightMapPaintBrushType::Erase);

	SetPrimaryFalloffType(EMeshSculptFalloffType::Smooth);



	InitializeSculptMeshFromTarget();

	UpdateShowHideProperties();


	// Copy weights from selected node to the preview mesh
	const int32 NumExpectedWeights = bHaveDynamicMeshToWeightConversion ? WeightToDynamicMesh.Num() : GetSculptMesh()->MaxVertexID();
	TArray<float> CurrentWeights;
	CurrentWeights.SetNumZeroed(NumExpectedWeights);
	WeightMapNodeToUpdate->CalculateFinalVertexWeightValues(nullptr, InputWeightMap, TArrayView<float>(CurrentWeights));

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 WeightID = 0; WeightID < CurrentWeights.Num(); ++WeightID)
		{
			for (const int32 VertexID : WeightToDynamicMesh[WeightID])
			{
				ActiveWeightMap->SetValue(VertexID, &CurrentWeights[WeightID]);
			}
		}
	}
	else
	{
		for (int32 VertexID = 0; VertexID < CurrentWeights.Num(); ++VertexID)
		{
			ActiveWeightMap->SetValue(VertexID, &CurrentWeights[VertexID]);
		}
	}

	// Initialize vertex colors from attribute layer values
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);

	PostSetupCheck();

	GetToolManager()->PostInvalidation();
}

void UClothEditorWeightMapPaintTool::InitializeSculptMeshFromTarget()
{
	InitializeSculptMeshComponent(DynamicMeshComponent, PreviewMeshActor);

	// assign materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	FDynamicMesh3* Mesh = GetSculptMesh();
	Mesh->EnableVertexColors(FVector3f::One());
	Mesh->Attributes()->EnablePrimaryColors();
	Mesh->Attributes()->PrimaryColors()->CreatePerVertex(0.f);
	FAxisAlignedBox3d Bounds = Mesh->GetBounds(true);

	TFuture<void> PrecomputeFuture = Async(UE::Chaos::ClothAsset::Private::WeightPaintToolAsyncExecTarget, [this]()
	{
		PrecomputeFilterData();
	});

	TFuture<void> OctreeFuture = Async(UE::Chaos::ClothAsset::Private::WeightPaintToolAsyncExecTarget, [Mesh, &Bounds, this]()
	{
		// initialize dynamic octree
		if (Mesh->TriangleCount() > 100000)
		{
			Octree.RootDimension = Bounds.MaxDim() / 10.0;
			Octree.SetMaxTreeDepth(4);
		}
		else
		{
			Octree.RootDimension = Bounds.MaxDim();
			Octree.SetMaxTreeDepth(8);
		}
		Octree.Initialize(Mesh);
	});

	// initialize render decomposition
	TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
	FMeshRenderDecomposition::BuildChunkedDecomposition(Mesh, &MaterialSet, *Decomp);
	Decomp->BuildAssociations(Mesh);
	DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));

	// initialize brush radius range interval, brush properties
	InitializeBrushSizeRange(Bounds);

	// Setup DynamicMeshToWeight conversion and get Input weight map (if it exists)
	InputWeightMap = TConstArrayView<float>();

	if (DataflowContextObject)
	{
		ensure(DataflowContextObject->IsUsingInputCollection());
		if (TSharedPtr<const FManagedArrayCollection> ClothCollection = DataflowContextObject->GetSelectedCollection())
		{
			using namespace UE::Chaos::ClothAsset;
			const FNonManifoldMappingSupport NonManifoldMapping(*Mesh);

			const bool bHasNonManifoldMapping = NonManifoldMapping.IsNonManifoldVertexInSource();
			const bool bHas2D3DConversion = DataflowViewModeToClothViewMode(DataflowContextObject->GetConstructionViewMode()) == EClothPatternVertexType::Sim2D;

			bHaveDynamicMeshToWeightConversion = bHasNonManifoldMapping || bHas2D3DConversion;

			FCollectionClothConstFacade Cloth(ClothCollection.ToSharedRef());
			check(Cloth.IsValid());
			if (bHasNonManifoldMapping)
			{
				const TConstArrayView<int32> SimVertex3DLookup = Cloth.GetSimVertex3DLookup();

				DynamicMeshToWeight.SetNumUninitialized(Mesh->VertexCount());
				WeightToDynamicMesh.Reset();
				WeightToDynamicMesh.SetNum(Cloth.GetNumSimVertices3D());
				for (int32 DynamicMeshVert = 0; DynamicMeshVert < Mesh->VertexCount(); ++DynamicMeshVert)
				{
					DynamicMeshToWeight[DynamicMeshVert] = NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert);
					if (bHas2D3DConversion)
					{
						DynamicMeshToWeight[DynamicMeshVert] = SimVertex3DLookup[DynamicMeshToWeight[DynamicMeshVert]];
					}
					WeightToDynamicMesh[DynamicMeshToWeight[DynamicMeshVert]].Add(DynamicMeshVert);
				}
			}
			else if (bHas2D3DConversion)
			{
				DynamicMeshToWeight = Cloth.GetSimVertex3DLookup();
				WeightToDynamicMesh = Cloth.GetSimVertex2DLookup();
			}

			const bool bIsRenderMode = (WeightMapNodeToUpdate->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render);

			// Find the map if it exists.
			if (TSharedPtr<UE::Dataflow::FEngineContext> DataflowContext = DataflowContextObject->GetDataflowContext())
			{
				const FName InputName = WeightMapNodeToUpdate->GetInputName(*DataflowContext);
				if (bIsRenderMode)
				{
					InputWeightMap = Cloth.GetUserDefinedAttribute<float>(InputName, ClothCollectionGroup::RenderVertices);
				}
				else
				{
					InputWeightMap = Cloth.GetWeightMap(InputName);
				}
			}
		}
	}

	PrecomputeFuture.Wait();
	OctreeFuture.Wait();

	// Create an attribute layer to temporarily paint into
	const int NumAttributeLayers = Mesh->Attributes()->NumWeightLayers();
	Mesh->Attributes()->SetNumWeightLayers(NumAttributeLayers + 1);
	ActiveWeightMap = Mesh->Attributes()->GetWeightLayer(NumAttributeLayers);
	ActiveWeightMap->SetName(FName("PaintLayer"));

	// Setup support for hiding specific triangles
	DynamicMeshComponent->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			return PendingHiddenTriangles.Contains(TriangleID) || HiddenTriangles.Contains(TriangleID);
		});
	DynamicMeshComponent->SetSecondaryBuffersVisibility(false);

	// Rebuild mechanics that depend on Mesh topology
	constexpr bool bAutoBuild = true;
	GradientSelectionTopology = MakeUnique<UE::Geometry::FTriangleGroupTopology>(DynamicMeshComponent->GetMesh(), bAutoBuild);
	MeshSpatial = MakeUnique<UE::Geometry::FDynamicMeshAABBTree3>(DynamicMeshComponent->GetMesh(), bAutoBuild);
	PolygonSelectionMechanic->Initialize(DynamicMeshComponent, GradientSelectionTopology.Get(), [this]() { return MeshSpatial.Get(); });
}

void UClothEditorWeightMapPaintTool::UpdateShowHideProperties()
{
	if (DataflowContextObject)
	{
		ensure(DataflowContextObject->IsUsingInputCollection());
		if (TSharedPtr<const FManagedArrayCollection> ClothCollection = DataflowContextObject->GetSelectedCollection())
		{
			using namespace UE::Chaos::ClothAsset;

			FCollectionClothConstFacade Cloth(ClothCollection.ToSharedRef());
			check(Cloth.IsValid());

			const bool bIsRenderMode = WeightMapNodeToUpdate
				? (WeightMapNodeToUpdate->MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render)
				: true;

			const int32 NumPatterns = bIsRenderMode ? Cloth.GetNumRenderPatterns() : Cloth.GetNumSimPatterns();

			PatternTriangleOffsetAndNum.SetNum(NumPatterns);

			TSet<int32> NonEmptyPatternIDs;

			for (int32 PatternIndex = 0; PatternIndex < NumPatterns; ++PatternIndex)
			{
				TPair<int32, int32>& OffsetAndNum = PatternTriangleOffsetAndNum[PatternIndex];
				if (bIsRenderMode)
				{
					FCollectionClothRenderPatternConstFacade RenderPattern = Cloth.GetRenderPattern(PatternIndex);
					OffsetAndNum.Key = RenderPattern.GetRenderFacesOffset();
					OffsetAndNum.Value = RenderPattern.GetNumRenderFaces();
				}
				else
				{
					FCollectionClothSimPatternConstFacade SimPattern = Cloth.GetSimPattern(PatternIndex);
					OffsetAndNum.Key = SimPattern.GetSimFacesOffset();
					OffsetAndNum.Value = SimPattern.GetNumSimFaces();
				}

				if (OffsetAndNum.Value > 0)
				{
					NonEmptyPatternIDs.Add(PatternIndex);
				}
			}

			// Initialize the ShowPatterns map from found pattern indices
			ShowHideProperties->ShowPatterns.Reset();
			for (const int32 PatternID : NonEmptyPatternIDs)
			{
				ShowHideProperties->ShowPatterns.Add({ PatternID, false });
			}
		}
	}
}


void UClothEditorWeightMapPaintTool::NotifyTargetChanged()
{
	//
	// The target mesh has changed due to a view mode change. We will attempt to transfer the current in-progress paint values to the new mesh.
	// First, temporarily save the existing weights from the paint layer on the mesh.
	// 

	TArray<float> SavedWeights;
	GetCurrentWeightMap(SavedWeights);

	if (bHaveDynamicMeshToWeightConversion)
	{
		TArray<float> MappedWeights;
		MappedWeights.Init(0.f, WeightToDynamicMesh.Num());
		for (int32 DynamicMeshIdx = 0; DynamicMeshIdx < SavedWeights.Num(); ++DynamicMeshIdx)
		{
			MappedWeights[DynamicMeshToWeight[DynamicMeshIdx]] = SavedWeights[DynamicMeshIdx];
		}

		SavedWeights = MoveTemp(MappedWeights);
	}


	//
	// Now re-initialize everything that depends on the mesh
	//

	InitializeSculptMeshFromTarget();

	UpdateShowHideProperties();

	//
	// Copy saved values back to the new preview mesh
	//

	checkf(ActiveWeightMap, TEXT("UClothEditorWeightMapPaintTool: no ActiveWeightMap after re-initializing the preview mesh"));

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 WeightID = 0; WeightID < SavedWeights.Num(); ++WeightID)
		{
			for (const int32 VertexID : WeightToDynamicMesh[WeightID])
			{
				ActiveWeightMap->SetValue(VertexID, &SavedWeights[WeightID]);
			}
		}
	}
	else
	{
		for (int32 VertexID = 0; VertexID < SavedWeights.Num(); ++VertexID)
		{
			ActiveWeightMap->SetValue(VertexID, &SavedWeights[VertexID]);
		}
	}

	//
	// Update visualization
	//

	checkf(DynamicMeshComponent, TEXT("UClothEditorWeightMapPaintTool: no preview mesh after the tool target changed"));

	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);

	MeshElementsDisplay->NotifyMeshChanged();

	GetToolManager()->PostInvalidation();

	PostSetupCheck();
}


void UClothEditorWeightMapPaintTool::PostSetupCheck() const
{
	check(WeightMapNodeToUpdate);

	check(PreviewMeshActor);
	check(DynamicMeshComponent);
	check(DynamicMeshComponent->GetAttachParent() == PreviewMeshActor->GetRootComponent());
	check(DynamicMeshComponent->GetMesh() == GetSculptMesh());

	check(ActiveWeightMap);
	check(ActiveWeightMap->GetParent() == GetSculptMesh());

	check(Octree.Mesh == GetSculptMesh());

	check(MeshSpatial);
	check(MeshSpatial->GetMesh() == GetSculptMesh());

	check(TriNormals.Num() == GetSculptMesh()->MaxTriangleID());
	check(UVSeamEdges.Num() == GetSculptMesh()->MaxEdgeID());
	check(NormalSeamEdges.Num() == GetSculptMesh()->MaxEdgeID());

	check(PolygonSelectionMechanic);
	check(PolyLassoMechanic);
	check(GradientSelectionTopology);
	check(GradientSelectionTopology->GetMesh() == GetSculptMesh());

	check(BrushProperties);
	check(ToolPropertyObjects.Contains(BrushProperties));
	check(GizmoProperties);
	check(ToolPropertyObjects.Contains(GizmoProperties));
	check(ViewProperties);
	check(ToolPropertyObjects.Contains(ViewProperties));
	check(UpdateWeightMapProperties);
	check(ToolPropertyObjects.Contains(UpdateWeightMapProperties));
	check(FilterProperties);
	check(ToolPropertyObjects.Contains(FilterProperties));
	check(PaintBrushOpProperties);
	check(ToolPropertyObjects.Contains(PaintBrushOpProperties));
	check(SmoothBrushOpProperties);
	check(ToolPropertyObjects.Contains(SmoothBrushOpProperties));
	check(EraseBrushOpProperties);
	check(ToolPropertyObjects.Contains(EraseBrushOpProperties));
	check(ActionsProps);
	check(ToolPropertyObjects.Contains(ActionsProps));
	check(ShowHideProperties);
	check(ToolPropertyObjects.Contains(ShowHideProperties));

	check(MeshElementsDisplay);
}

void UClothEditorWeightMapPaintTool::InitializeBrushSizeRange(const UE::Geometry::FAxisAlignedBox3d& TargetBounds)
{
	const double MaxDimension = FMath::Max(0.1, TargetBounds.MaxDim());

	// Max brush size is the next power of 2 greater than MaxDimension. This allows the mesh size to change somewhat and still keep the same brush size, but still allow 
	// the brush size to adapt to meshes of vastly different scales (e.g. switching from world coordinates to texture coordinates.)
	constexpr double StepSize = 2.0;
	const double MaxBrushSize = FMath::Pow(StepSize, FMath::CeilToDouble(FMath::LogX(StepSize, MaxDimension)));

	BrushRelativeSizeRange = FInterval1d(5e-5 * MaxBrushSize, MaxBrushSize);
	BrushProperties->BrushSize.InitializeWorldSizeRange(TInterval<float>((float)BrushRelativeSizeRange.Min, (float)BrushRelativeSizeRange.Max));
	CalculateBrushRadius();
}

void UClothEditorWeightMapPaintTool::NextBrushModeAction()
{
	constexpr uint8 NumCyclableBrushes = 2;		// Don't cycle to the hidden Erase brush
	FilterProperties->PrimaryBrushType = static_cast<EClothEditorWeightMapPaintBrushType>((static_cast<uint8>(FilterProperties->PrimaryBrushType) + 1) % NumCyclableBrushes);
}

void UClothEditorWeightMapPaintTool::PreviousBrushModeAction()
{
	constexpr uint8 NumCyclableBrushes = 2;		// Don't cycle to the hidden Erase brush
	const uint8 CurrentBrushType = static_cast<uint8>(FilterProperties->PrimaryBrushType);
	const uint8 NewBrushType = CurrentBrushType == 0 ? NumCyclableBrushes - 1 : (CurrentBrushType - 1) % NumCyclableBrushes;
	FilterProperties->PrimaryBrushType = static_cast<EClothEditorWeightMapPaintBrushType>(NewBrushType);
}

void UClothEditorWeightMapPaintTool::IncreaseBrushSpeedAction()		// Actually increases AttributeValue
{
	const double CurrentValue = FilterProperties->AttributeValue;
	FilterProperties->AttributeValue = FMath::Clamp(CurrentValue + 0.05, 0.0, 1.0);
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UClothEditorWeightMapPaintTool::DecreaseBrushSpeedAction()		// Actually decreases AttributeValue
{
	const double CurrentValue = FilterProperties->AttributeValue;
	FilterProperties->AttributeValue = FMath::Clamp(CurrentValue - 0.05, 0.0, 1.0);
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UClothEditorWeightMapPaintTool::SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject)
{
	DataflowContextObject = InDataflowContextObject;
}

void UClothEditorWeightMapPaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);
	}

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("ClothEditorWeightMapPaintTool2"));
	}
	MeshElementsDisplay->Disconnect();

	FilterProperties->SaveProperties(this);

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->Shutdown();
		PolygonSelectionMechanic = nullptr;
	}

	UMeshSculptToolBase::Shutdown(ShutdownType);

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}
}


void UClothEditorWeightMapPaintTool::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("WeightPaintToolTransactionName", "Paint Weights"));

	UpdateSelectedNode();

	GetToolManager()->EndUndoTransaction();
}


void UClothEditorWeightMapPaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UMeshSculptToolBase::RegisterActions(ActionSet);
	
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 500,
		TEXT("PickWeightValueUnderCursor"),
		LOCTEXT("PickWeightValueUnderCursor", "Pick Weight Value"),
		LOCTEXT("PickWeightValueUnderCursorTooltip", "Set the active weight painting value to that currently under the cursor"),
		EModifierKey::Shift, EKeys::G,
		[this]() { bPendingPickWeight = true; });


	// E/W are overridden to decrease/increase the AttributeValue property. Use shift-E/shift-W to increment by smaller amount

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 503,
		TEXT("WeightMapPaintIncreaseValueSmallStep"),
		LOCTEXT("WeightMapPaintIncreaseValueSmallStep", "Increase Value"),
		LOCTEXT("WeightMapPaintIncreaseValueSmallStepTooltip", "Increase Value (small increment)"),
		EModifierKey::Shift, EKeys::E,
		[this]()
		{
			const double CurrentValue = FilterProperties->AttributeValue;
			FilterProperties->AttributeValue = FMath::Clamp(CurrentValue + 0.005, 0.0, 1.0);
			NotifyOfPropertyChangeByTool(FilterProperties);
		});

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 504,
		TEXT("WeightMapPaintDecreaseValueSmallStep"),
		LOCTEXT("WeightMapPaintDecreaseValueSmallStep", "Decrease Value"),
		LOCTEXT("WeightMapPaintDecreaseValueSmallStepTooltip", "Decrease Value (small increment)"),
		EModifierKey::Shift, EKeys::W,
		[this]()
		{
			const double CurrentValue = FilterProperties->AttributeValue;
			FilterProperties->AttributeValue = FMath::Clamp(CurrentValue - 0.005, 0.0, 1.0);
			NotifyOfPropertyChangeByTool(FilterProperties);
		});

};


TUniquePtr<FMeshSculptBrushOp>& UClothEditorWeightMapPaintTool::GetActiveBrushOp()
{
	if (GetInEraseStroke())
	{
		return SecondaryBrushOp;
	}
	else
	{
		return PrimaryBrushOp;
	}
}


void UClothEditorWeightMapPaintTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}


void UClothEditorWeightMapPaintTool::IncreaseBrushRadiusAction()
{
	Super::IncreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UClothEditorWeightMapPaintTool::DecreaseBrushRadiusAction()
{
	Super::DecreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UClothEditorWeightMapPaintTool::IncreaseBrushRadiusSmallStepAction()
{
	Super::IncreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UClothEditorWeightMapPaintTool::DecreaseBrushRadiusSmallStepAction()
{
	Super::DecreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}


bool UClothEditorWeightMapPaintTool::IsInBrushSubMode() const
{
	return FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Brush
		|| FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Fill
	    || FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::HideTriangles;
}


void UClothEditorWeightMapPaintTool::OnBeginStroke(const FRay& WorldRay)
{
	if (!ActiveWeightMap)
	{
		return;
	}

	UpdateBrushPosition(WorldRay);

	if (PaintBrushOpProperties)
	{
		PaintBrushOpProperties->AttributeValue = FilterProperties->AttributeValue;
		PaintBrushOpProperties->Strength = FilterProperties->Strength * FilterProperties->Strength;
	}
	if (EraseBrushOpProperties)
	{
		EraseBrushOpProperties->AttributeValue = 0.0;
	}
	if (SmoothBrushOpProperties)
	{
		SmoothBrushOpProperties->Strength = FilterProperties->Strength * FilterProperties->Strength;
		SmoothBrushOpProperties->Falloff = FilterProperties->Falloff;
	}

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = GetBrushFrameWorld();
	LastStamp.LocalFrame = GetBrushFrameLocal();
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActivePressure() * GetCurrentBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	FSculptBrushOptions SculptOptions;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	AccumulatedTriangleROI.Reset();

	// begin change here? or wait for first stamp?
	BeginChange();
}

void UClothEditorWeightMapPaintTool::OnEndStroke()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	if (PendingHiddenTriangles.Num() > 0)
	{
		HiddenTriangles.Append(PendingHiddenTriangles);
		PendingHiddenTriangles.Reset();
		MeshElementsDisplay->NotifyMeshChanged();
		DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
	}

	UpdateVertexColorOverlay(&TriangleROI);
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);

	// close change record
	EndChange();
}

void UClothEditorWeightMapPaintTool::OnCancelStroke()
{
	GetActiveBrushOp()->CancelStroke();
	ActiveChangeBuilder.Reset();
}



void UClothEditorWeightMapPaintTool::UpdateROI(const FSculptBrushStamp& BrushStamp)
{
	SCOPE_CYCLE_COUNTER(WeightMapPaintTool_UpdateROI);

	const FVector3d& BrushPos = BrushStamp.LocalFrame.Origin;
	const FDynamicMesh3* Mesh = GetSculptMesh();
	float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	TriangleROI.Reset();

	int32 CenterTID = GetBrushTriangleID();
	if (Mesh->IsTriangle(CenterTID))
	{
		TriangleROI.Add(CenterTID);
	}

	FVector3d CenterNormal = Mesh->IsTriangle(CenterTID) ? TriNormals[CenterTID] : FVector3d::One();		// One so that normal check always passes
	
	bool bUseAngleThreshold = FilterProperties->AngleThreshold < 180.0f;
	double DotAngleThreshold = FMathd::Cos(FilterProperties->AngleThreshold * FMathd::DegToRad);
	bool bStopAtUVSeams = FilterProperties->bUVSeams;
	bool bStopAtNormalSeams = FilterProperties->bNormalSeams;

	auto CheckEdgeCriteria = [&](int32 t1, int32 t2) -> bool
	{
		if (bUseAngleThreshold == false || CenterNormal.Dot(TriNormals[t2]) > DotAngleThreshold)
		{
			int32 eid = Mesh->FindEdgeFromTriPair(t1, t2);
			if (bStopAtUVSeams == false || UVSeamEdges[eid] == false)
			{
				if (bStopAtNormalSeams == false || NormalSeamEdges[eid] == false)
				{
					return true;
				}
			}
		}
		return false;
	};

	bool bFill = (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Fill);

	if (Mesh->IsTriangle(CenterTID))
	{
		TArray<int32> StartROI;
		StartROI.Add(CenterTID);
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
			[&](int t1, int t2) 
		{ 
			if ((Mesh->GetTriCentroid(t2) - BrushPos).SquaredLength() < RadiusSqr)
			{
				return CheckEdgeCriteria(t1, t2);
			}
			return false;
		});
	}

	if (bFill)
	{
		TArray<int32> StartROI;
		for (int32 tid : TriangleROI)
		{
			StartROI.Add(tid);
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
														   [&](int t1, int t2)
		{
			return CheckEdgeCriteria(t1, t2);
		});
	}

	// construct ROI vertex set
	VertexSetBuffer.Reset();
	for (int32 tid : TriangleROI)
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		VertexSetBuffer.Add(Tri.A);  VertexSetBuffer.Add(Tri.B);  VertexSetBuffer.Add(Tri.C);
	}
	
	// apply visibility filter
	if (FilterProperties->VisibilityFilter != EClothEditorWeightMapPaintVisibilityType::None)
	{
		TArray<int32> ResultBuffer;
		ApplyVisibilityFilter(VertexSetBuffer, TempROIBuffer, ResultBuffer);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		// Find triangles whose vertices map to the same welded vertex as any vertex in VertexSetBuffer and add them to TriangleROI

		for (const int VertexID : VertexSetBuffer)
		{
			for (const int OtherVertexID : WeightToDynamicMesh[DynamicMeshToWeight[VertexID]])
			{
				if (OtherVertexID != VertexID)
				{
					Mesh->EnumerateVertexTriangles(OtherVertexID, [this](int32 AdjacentTri)
					{
						TriangleROI.Add(AdjacentTri);
					});
				}
			}
		}
	}

	// If we are Smoothing, expand the set of vertices to consider. Otherwise vertices near the brush bounds will not use the expected neighborhood to get an average weight.
	const bool bExpandVertexROI = (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Brush && FilterProperties->PrimaryBrushType == EClothEditorWeightMapPaintBrushType::Smooth);
	if (bExpandVertexROI)
	{
		TSet<int32> NewVertexSetBuffer = VertexSetBuffer;
		for (const int32 Vert : VertexSetBuffer)
		{
			for (const int32 NeighborVert : Mesh->VtxVerticesItr(Vert))
			{
				NewVertexSetBuffer.Add(NeighborVert);
			}
		}
		VertexSetBuffer = MoveTemp(NewVertexSetBuffer);
	}


	VertexROI.SetNum(0, EAllowShrinking::No);
	//TODO: If we paint a 2D projection of UVs, these will need to be the 2D vertices not the 3D original mesh vertices
	BufferUtil::AppendElements(VertexROI, VertexSetBuffer);

	// construct ROI triangle and weight buffers
	ROITriangleBuffer.Reserve(TriangleROI.Num());
	ROITriangleBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : TriangleROI)
	{
		ROITriangleBuffer.Add(tid);
	}
	ROIWeightValueBuffer.SetNum(VertexROI.Num(), EAllowShrinking::No);
	SyncWeightBufferWithMesh(Mesh);
}

bool UClothEditorWeightMapPaintTool::UpdateStampPosition(const FRay& WorldRay)
{
	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		UpdateBrushPositionOnSculptMesh(WorldRay, true);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		UpdateBrushPositionOnActivePlane(WorldRay);
		break;
	}

	if (UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	CurrentStamp = LastStamp;
	CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.WorldFrame = GetBrushFrameWorld();
	CurrentStamp.LocalFrame = GetBrushFrameLocal();
	CurrentStamp.Power = GetActivePressure() * GetCurrentBrushStrength();

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	FVector3d MoveDelta = CurrentStamp.LocalFrame.Origin - CurrentStamp.PrevLocalFrame.Origin;

	if (UseBrushOp->IgnoreZeroMovements() && MoveDelta.SquaredLength() < 0.1 * CurrentBrushRadius)
	{
		return false;
	}

	return true;
}


bool UClothEditorWeightMapPaintTool::ApplyStamp()
{
	SCOPE_CYCLE_COUNTER(WeightMapPaintToolApplyStamp);

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	// yuck
	FMeshVertexWeightMapEditBrushOp* WeightBrushOp = (FMeshVertexWeightMapEditBrushOp*)UseBrushOp.Get();

	if (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Brush)
	{
		WeightBrushOp->bApplyRadiusLimit = true;
	}
	else
	{ 
		WeightBrushOp->bApplyRadiusLimit = false;
	}

	bool bUpdated = false;
	if (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Fill)
	{
		FDynamicMesh3* Mesh = GetSculptMesh();
		WeightBrushOp->ApplyStampByVertices(Mesh, CurrentStamp, VertexROI, ROIWeightValueBuffer);
		bUpdated = SyncMeshWithWeightBuffer(Mesh);
	}
	else
	{
		bool bAnyModified = false;
		for (int32 TID : TriangleROI)
		{
			bool bModified = false;
			PendingHiddenTriangles.Add(TID, &bModified);
			bAnyModified = bAnyModified || bModified;
		}

		if (bAnyModified)
		{
			DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
		}
	}
	

	LastStamp = CurrentStamp;
	LastStamp.TimeStamp = FDateTime::Now();

	return bUpdated;
}




bool UClothEditorWeightMapPaintTool::SyncMeshWithWeightBuffer(FDynamicMesh3* Mesh)
{
	int NumModified = 0;
	const int32 NumT = VertexROI.Num();
	if (ActiveWeightMap)
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		for (int32 k = 0; k < NumT; ++k)
		{
			int VertIdx = VertexROI[k];
			double CurWeight = GetCurrentWeightValue(VertIdx);

			if (ROIWeightValueBuffer[k] != CurWeight)
			{
				const float NewValue = ROIWeightValueBuffer[k];

				if (bHaveDynamicMeshToWeightConversion)
				{
					for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertIdx]])
					{
						float PrevValue;
						ActiveWeightMap->GetValue(Idx, &PrevValue);

						ensure(DynamicMeshToWeight[VertIdx] == MeshIndexToNodeIndex(Idx));
						ActiveChangeBuilder->UpdateValue(DynamicMeshToWeight[VertIdx], PrevValue, NewValue);
						
						ActiveWeightMap->SetValue(Idx, &NewValue);
					}
				}
				else
				{
					float PrevValue;
					ActiveWeightMap->GetValue(VertIdx, &PrevValue);

					ensure(VertIdx == MeshIndexToNodeIndex(VertIdx));
					ActiveChangeBuilder->UpdateValue(VertIdx, PrevValue, NewValue);

					ActiveWeightMap->SetValue(VertIdx, &NewValue);
				}
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

bool UClothEditorWeightMapPaintTool::SyncWeightBufferWithMesh(const FDynamicMesh3* Mesh)
{
	int NumModified = 0;
	const int32 NumT = VertexROI.Num();
	if (ActiveWeightMap)
	{
		// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
		for (int32 k = 0; k < NumT; ++k)
		{
			int VertIdx = VertexROI[k];
			double CurWeight = GetCurrentWeightValue(VertIdx);
			if (ROIWeightValueBuffer[k] != CurWeight)
			{
				ROIWeightValueBuffer[k] = CurWeight;
				NumModified++;
			}
		}
	}
	return (NumModified > 0);
}

template<typename RealType>
static bool FindPolylineSelfIntersection(
	const TArray<UE::Math::TVector2<RealType>>& Polyline, 
	UE::Math::TVector2<RealType>& IntersectionPointOut, 
	FIndex2i& IntersectionIndexOut,
	bool bParallel = true)
{
	int32 N = Polyline.Num();
	std::atomic<bool> bSelfIntersects(false);
	ParallelFor(N - 1, [&](int32 i)
	{
		TSegment2<RealType> SegA(Polyline[i], Polyline[i + 1]);
		for (int32 j = i + 2; j < N - 1 && bSelfIntersects == false; ++j)
		{
			TSegment2<RealType> SegB(Polyline[j], Polyline[j + 1]);
			if (SegA.Intersects(SegB) && bSelfIntersects == false)		
			{
				bool ExpectedValue = false;
				if (std::atomic_compare_exchange_strong(&bSelfIntersects, &ExpectedValue, true))
				{
					TIntrSegment2Segment2<RealType> Intersection(SegA, SegB);
					Intersection.Find();
					IntersectionPointOut = Intersection.Point0;
					IntersectionIndexOut = FIndex2i(i, j);
					return;
				}
			}
		}
	}, (bParallel) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread );

	return bSelfIntersects;
}



template<typename RealType>
static bool FindPolylineSegmentIntersection(
	const TArray<UE::Math::TVector2<RealType>>& Polyline,
	const TSegment2<RealType>& Segment,
	UE::Math::TVector2<RealType>& IntersectionPointOut,
	int& IntersectionIndexOut)
{

	int32 N = Polyline.Num();
	for (int32 i = 0; i < N-1; ++i)
	{
		TSegment2<RealType> PolySeg(Polyline[i], Polyline[i + 1]);
		if (Segment.Intersects(PolySeg))
		{
			TIntrSegment2Segment2<RealType> Intersection(Segment, PolySeg);
			Intersection.Find();
			IntersectionPointOut = Intersection.Point0;
			IntersectionIndexOut = i;
			return true;
		}
	}
	return false;
}



bool ApproxSelfClipPolyline(TArray<FVector2f>& Polyline)
{
	int32 N = Polyline.Num();

	// handle already-closed polylines
	if (Distance(Polyline[0], Polyline[N-1]) < 0.0001f)
	{
		return true;
	}

	FVector2f IntersectPoint;
	FIndex2i IntersectionIndex(-1, -1);
	bool bSelfIntersects = FindPolylineSelfIntersection(Polyline, IntersectPoint, IntersectionIndex);
	if (bSelfIntersects)
	{
		TArray<FVector2f> NewPolyline;
		NewPolyline.Add(IntersectPoint);
		for (int32 i = IntersectionIndex.A; i <= IntersectionIndex.B; ++i)
		{
			NewPolyline.Add(Polyline[i]);
		}
		NewPolyline.Add(IntersectPoint);
		Polyline = MoveTemp(NewPolyline);
		return true;
	}


	FVector2f StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
	FLine2f StartLine(Polyline[0], StartDirOut);
	FVector2f EndDirOut = UE::Geometry::Normalized(Polyline[N - 1] - Polyline[N - 2]);
	FLine2f EndLine(Polyline[N - 1], EndDirOut);
	FIntrLine2Line2f LineIntr(StartLine, EndLine);
	bool bIntersects = false;
	if (LineIntr.Find())
	{
		bIntersects = LineIntr.IsSimpleIntersection() && (LineIntr.Segment1Parameter > 0) && (LineIntr.Segment2Parameter > 0);
		if (bIntersects)
		{
			Polyline.Add(StartLine.PointAt(LineIntr.Segment1Parameter));
			Polyline.Add(StartLine.Origin);
			return true;
		}
	}


	FAxisAlignedBox2f Bounds;
	for (const FVector2f& P : Polyline)
	{
		Bounds.Contain(P);
	}
	float Size = Bounds.DiagonalLength();

	FVector2f StartPos = Polyline[0] + 0.001f * StartDirOut;
	if (FindPolylineSegmentIntersection(Polyline, FSegment2f(StartPos, StartPos + 2*Size*StartDirOut), IntersectPoint, IntersectionIndex.A))
	{
		return true;
	}

	FVector2f EndPos = Polyline[N-1] + 0.001f * EndDirOut;
	if (FindPolylineSegmentIntersection(Polyline, FSegment2f(EndPos, EndPos + 2*Size*EndDirOut), IntersectPoint, IntersectionIndex.A))
	{
		return true;
	}

	return false;
}



void UClothEditorWeightMapPaintTool::OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled)
{
	// construct polyline
	TArray<FVector2f> Polyline;
	for (FVector2D Pos : Lasso.Polyline)
	{
		Polyline.Add((FVector2f)Pos);
	}
	int32 N = Polyline.Num();
	if (N < 2)
	{
		return;
	}

	// Try to clip polyline to be closed, or closed-enough for winding evaluation to work.
	// If that returns false, the polyline is "too open". In that case we will extend
	// outwards from the endpoints and then try to create a closed very large polygon
	if (ApproxSelfClipPolyline(Polyline) == false)
	{
		FVector2f StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		FLine2f StartLine(Polyline[0], StartDirOut);
		FVector2f EndDirOut = UE::Geometry::Normalized(Polyline[N-1] - Polyline[N-2]);
		FLine2f EndLine(Polyline[N-1], EndDirOut);

		// if we did not intersect, we are in ambiguous territory. Check if a segment along either end-direction
		// intersects the polyline. If it does, we have something like a spiral and will be OK. 
		// If not, make a closed polygon by interpolating outwards from each endpoint, and then in perp-directions.
		FPolygon2f Polygon(Polyline);
		float PerpSign = Polygon.IsClockwise() ? -1.0 : 1.0;

		Polyline.Insert(StartLine.PointAt(10000.0f), 0);
		Polyline.Insert(Polyline[0] + 1000 * PerpSign * UE::Geometry::PerpCW(StartDirOut), 0);

		Polyline.Add(EndLine.PointAt(10000.0f));
		Polyline.Add(Polyline.Last() + 1000 * PerpSign * UE::Geometry::PerpCW(EndDirOut));
		FVector2f StartPos = Polyline[0];
		Polyline.Add(StartPos);		// close polyline (cannot use Polyline[0] in case Add resizes!)
	}

	N = Polyline.Num();

	// project each mesh vertex to view plane and evaluate winding integral of polyline
	const FDynamicMesh3* Mesh = GetSculptMesh();
	TempROIBuffer.SetNum(Mesh->MaxVertexID());
	ParallelFor(Mesh->MaxVertexID(), [&](int32 vid)
	{
		if (Mesh->IsVertex(vid))
		{
			FVector3d WorldPos = CurTargetTransform.TransformPosition(Mesh->GetVertex(vid));
			FVector2f PlanePos = (FVector2f)Lasso.GetProjectedPoint((FVector)WorldPos);

			double WindingSum = 0;
			FVector2f a = Polyline[0] - PlanePos, b = FVector2f::Zero();
			for (int32 i = 1; i < N; ++i)
			{
				b = Polyline[i] - PlanePos;
				WindingSum += (double)FMathf::Atan2(a.X*b.Y - a.Y*b.X, a.X*b.X + a.Y*b.Y);
				a = b;
			}
			WindingSum /= FMathd::TwoPi;
			bool bInside = FMathd::Abs(WindingSum) > 0.3;
			TempROIBuffer[vid] = bInside ? 1 : 0;
		}
		else
		{
			TempROIBuffer[vid] = -1;
		}
	});

	// convert to vertex selection, and then select fully-enclosed faces
	FMeshVertexSelection VertexSelection(Mesh);
	VertexSelection.SelectByVertexID([&](int32 vid) { return TempROIBuffer[vid] == 1; });

	double SetWeightValue = GetInEraseStroke() ? 0.0 : FilterProperties->AttributeValue;
	SetVerticesToWeightMap(VertexSelection.AsSet(), SetWeightValue, GetInEraseStroke());
}


void UClothEditorWeightMapPaintTool::ComputeGradient()
{
	if (!ensure(ActiveWeightMap))
	{
		UE_LOGF(LogTemp, Warning, "No active weight map");
		return;
	}

	BeginChange();

	const FDynamicMesh3* const Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 vid : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(vid);
	}

	for (const int32 VertexIndex : TempROIBuffer)
	{
		const FVector3d Vert = Mesh->GetVertex(VertexIndex);

		// (Copied from FClothPaintTool_Gradient::ApplyGradient)

		// Get distances
		// TODO: Look into surface distance instead of 3D distance? May be necessary for some complex shapes
		float DistanceToLowSq = MAX_flt;
		for (const int32& LowIndex : LowValueGradientVertexSelection.SelectedCornerIDs)
		{
			const FVector3d LowPoint = Mesh->GetVertex(LowIndex);
			const float DistanceSq = (LowPoint - Vert).SizeSquared();
			if (DistanceSq < DistanceToLowSq)
			{
				DistanceToLowSq = DistanceSq;
			}
		}

		float DistanceToHighSq = MAX_flt;
		for (const int32& HighIndex : HighValueGradientVertexSelection.SelectedCornerIDs)
		{
			const FVector3d HighPoint = Mesh->GetVertex(HighIndex);
			const float DistanceSq = (HighPoint - Vert).SizeSquared();
			if (DistanceSq < DistanceToHighSq)
			{
				DistanceToHighSq = DistanceSq;
			}
		}

		const float NewValue = FMath::LerpStable(FilterProperties->GradientLowValue, FilterProperties->GradientHighValue, DistanceToLowSq / (DistanceToLowSq + DistanceToHighSq));
		if (bHaveDynamicMeshToWeightConversion)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[VertexIndex]])
			{
				float PreviousValue;
				ActiveWeightMap->GetValue(Idx, &PreviousValue);
				
				ensure(MeshIndexToNodeIndex(Idx) == DynamicMeshToWeight[VertexIndex]);
				ActiveChangeBuilder->UpdateValue(DynamicMeshToWeight[VertexIndex], PreviousValue, NewValue);

				ActiveWeightMap->SetValue(Idx, &NewValue);
			}
		}
		else
		{
			float PreviousValue;
			ActiveWeightMap->GetValue(VertexIndex, &PreviousValue);

			ensure(MeshIndexToNodeIndex(VertexIndex) == VertexIndex);
			ActiveChangeBuilder->UpdateValue(VertexIndex, PreviousValue, NewValue);

			ActiveWeightMap->SetValue(VertexIndex, &NewValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();

}


void UClothEditorWeightMapPaintTool::OnSelectionModified()
{
	const bool bToolTypeIsGradient = (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Gradient);
	if (bToolTypeIsGradient && PolygonSelectionMechanic)
	{
		const FGroupTopologySelection NewSelection = PolygonSelectionMechanic->GetActiveSelection();

		const bool bSelectingLowValueGradientVertices = !GetCtrlToggle();
		if (bSelectingLowValueGradientVertices)
		{
			HighValueGradientVertexSelection.Remove(NewSelection);
			LowValueGradientVertexSelection = NewSelection;
		}
		else
		{
			LowValueGradientVertexSelection.Remove(NewSelection);
			HighValueGradientVertexSelection = NewSelection;
		}

		if (LowValueGradientVertexSelection.SelectedCornerIDs.Num() > 0 && HighValueGradientVertexSelection.SelectedCornerIDs.Num() > 0)
		{
			ComputeGradient();
		}

		constexpr bool bBroadcast = false;
		PolygonSelectionMechanic->SetSelection(FGroupTopologySelection(), bBroadcast);
	}
}


void UClothEditorWeightMapPaintTool::SetVerticesToWeightMap(const TSet<int32>& Vertices, double WeightValue, bool bIsErase)
{
	BeginChange();

	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 vid : Vertices)
	{
		TempROIBuffer.Add(vid);
	}

	if (HaveVisibilityFilter())
	{
		TArray<int32> VisibleVertices;
		VisibleVertices.Reserve(TempROIBuffer.Num());
		ApplyVisibilityFilter(TempROIBuffer, VisibleVertices);
		TempROIBuffer = MoveTemp(VisibleVertices);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 vid : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[vid]])
			{
				float PreviousValue;
				ActiveWeightMap->GetValue(Idx, &PreviousValue);

				ensure(MeshIndexToNodeIndex(Idx) == DynamicMeshToWeight[vid]);
				ActiveChangeBuilder->UpdateValue(DynamicMeshToWeight[vid], PreviousValue, WeightValue);

				ActiveWeightMap->SetValue(Idx, &WeightValue);
			}
		}
	}
	else
	{
		for (int32 vid : TempROIBuffer)
		{
			float PreviousValue;
			ActiveWeightMap->GetValue(vid, &PreviousValue);
			
			ensure(MeshIndexToNodeIndex(vid) == vid);
			ActiveChangeBuilder->UpdateValue(vid, PreviousValue, WeightValue);

			ActiveWeightMap->SetValue(vid, &WeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	

	EndChange();
	
}



bool UClothEditorWeightMapPaintTool::HaveVisibilityFilter() const
{
	return FilterProperties->VisibilityFilter != EClothEditorWeightMapPaintVisibilityType::None;
}


void UClothEditorWeightMapPaintTool::ApplyVisibilityFilter(TSet<int32>& Vertices, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer)
{
	ROIBuffer.SetNum(0, EAllowShrinking::No);
	ROIBuffer.Reserve(Vertices.Num());
	for (int32 vid : Vertices)
	{
		ROIBuffer.Add(vid);
	}
	
	OutputBuffer.Reset();
	ApplyVisibilityFilter(TempROIBuffer, OutputBuffer);

	Vertices.Reset();
	for (int32 vid : OutputBuffer)
	{
		Vertices.Add(vid);
	}
}

void UClothEditorWeightMapPaintTool::ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices)
{
	if (!HaveVisibilityFilter())
	{
		VisibleVertices = Vertices;
		return;
	}

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 NumVertices = Vertices.Num();

	VisibilityFilterBuffer.SetNum(NumVertices, EAllowShrinking::No);
	ParallelFor(NumVertices, [&](int32 idx)
	{
		VisibilityFilterBuffer[idx] = true;
		UE::Geometry::FVertexInfo VertexInfo;
		Mesh->GetVertex(Vertices[idx], VertexInfo, true, false, false);
		FVector3d Centroid = VertexInfo.Position;
		FVector3d FaceNormal = (FVector3d)VertexInfo.Normal;
		if (FaceNormal.Dot((Centroid - LocalEyePosition)) > 0)
		{
			VisibilityFilterBuffer[idx] = false;
		}
		if (FilterProperties->VisibilityFilter == EClothEditorWeightMapPaintVisibilityType::Unoccluded)
		{
			int32 HitTID = Octree.FindNearestHitObject(FRay3d(LocalEyePosition, UE::Geometry::Normalized(Centroid - LocalEyePosition)));
			if (HitTID != IndexConstants::InvalidID && Mesh->IsTriangle(HitTID))
			{
				// Check to see if our vertex has been occulded by another triangle.
				FIndex3i TriVertices = Mesh->GetTriangle(HitTID);
				if (TriVertices[0] != Vertices[idx] && TriVertices[1] != Vertices[idx] && TriVertices[2] != Vertices[idx])
				{
					VisibilityFilterBuffer[idx] = false;
				}
			}
		}
	});

	VisibleVertices.Reset();
	for (int32 k = 0; k < NumVertices; ++k)
	{
		if (VisibilityFilterBuffer[k])
		{
			VisibleVertices.Add(Vertices[k]);
		}
	}
}



int32 UClothEditorWeightMapPaintTool::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	if (!IsInBrushSubMode())
	{
		return IndexConstants::InvalidID;
	}

	int32 HitTID;
	const FDynamicMesh3* const Mesh = GetSculptMesh();

	if (FilterProperties->bHitBackFaces)
	{
		HitTID = Octree.FindNearestHitObject(LocalRay,
			[this](int TriangleID)
			{ 
				if (HiddenTriangles.Contains(TriangleID))
				{
					return false;
				}
				return true;
			});
	}
	else
	{
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		HitTID = Octree.FindNearestHitObject(LocalRay,
			[this, Mesh, &LocalEyePosition](int TriangleID) 
			{
				if (HiddenTriangles.Contains(TriangleID))
				{
					return false;
				}
				FVector3d Normal, Centroid;
				double Area;
				Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
				return Normal.Dot((Centroid - LocalEyePosition)) < 0;
			});
	}

	return HitTID;
}

int32 UClothEditorWeightMapPaintTool::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	check(false);
	return IndexConstants::InvalidID;
}

void UClothEditorWeightMapPaintTool::UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay)
{
	CurrentBaryCentricCoords = FVector3d::Zero();

	const FDynamicMesh3* const Mesh = GetSculptMesh();

	if (Mesh->IsTriangle(TriangleID))
	{
		FTriangle3d Triangle;
		Mesh->GetTriVertices(TriangleID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();
		CurrentBaryCentricCoords = Query.TriangleBaryCoords;
	}

}


bool UClothEditorWeightMapPaintTool::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false; 
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	if (bHit && UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	return bHit;
}




bool UClothEditorWeightMapPaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	PendingStampType = FilterProperties->PrimaryBrushType;

	if(ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);
	}
	return true;
}


void UClothEditorWeightMapPaintTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (PolyLassoMechanic)
	{
		// because the actual Weight change is deferred until mouse release, color the lasso to let the user know whether it will erase
		PolyLassoMechanic->LineColor = GetInEraseStroke() ? FLinearColor::Red : FLinearColor::Green;
		PolyLassoMechanic->DrawHUD(Canvas, RenderAPI);
	}

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->DrawHUD(Canvas, RenderAPI);
	}
}

void UClothEditorWeightMapPaintTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSculptToolBase::Render(RenderAPI);

	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->RenderMarquee(RenderAPI);

		const FViewCameraState RenderCameraState = RenderAPI->GetCameraState();
		GradientSelectionRenderer.BeginFrame(RenderAPI, RenderCameraState);

		const FTransform Transform = DynamicMeshComponent->GetComponentTransform();
		GradientSelectionRenderer.SetTransform(Transform);
			
		GradientSelectionRenderer.SetPointParameters(FLinearColor::Green, 1.0);
		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(LowValueGradientVertexSelection, &GradientSelectionRenderer, &RenderCameraState);

		GradientSelectionRenderer.SetPointParameters(FLinearColor::Red, 1.0);
		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(HighValueGradientVertexSelection, &GradientSelectionRenderer, &RenderCameraState);

		// Now the current unsaved selection
		if (GetCtrlToggle())
		{
			GradientSelectionRenderer.SetPointParameters(FLinearColor::Red, 1.0);
		}
		else
		{
			GradientSelectionRenderer.SetPointParameters(FLinearColor::Green, 1.0);
		}

		PolygonSelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector()->DrawSelection(PolygonSelectionMechanic->GetActiveSelection(), &GradientSelectionRenderer, &RenderCameraState);

		GradientSelectionRenderer.EndFrame();
	}
}


void UClothEditorWeightMapPaintTool::UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode)
{
	if (MaterialMode == EMeshEditingMaterialModes::VertexColor)
	{
		constexpr bool bUseTwoSidedMaterial = true;
		ActiveOverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager(), bUseTwoSidedMaterial);
		if (ensure(ActiveOverrideMaterial != nullptr))
		{
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
			ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
		}
		GetSculptMeshComponent()->SetShadowsEnabled(false);
	}
	else
	{
		UMeshSculptToolBase::UpdateMaterialMode(MaterialMode);
	}
}

void UClothEditorWeightMapPaintTool::OnTick(float DeltaTime)
{
	UMeshSculptToolBase::OnTick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);

	const bool bIsLasso = (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::PolyLasso);
	PolyLassoMechanic->SetIsEnabled(bIsLasso);

	const bool bIsGradient = (FilterProperties->SubToolType == EClothEditorWeightMapPaintInteractionType::Gradient);
	PolygonSelectionMechanic->SetIsEnabled(bIsGradient);

	check(!(bIsLasso && bIsGradient));

	ConfigureIndicator(false);
	SetIndicatorVisibility(!bIsLasso && !bIsGradient);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EClothEditorWeightMapPaintToolActions::NoAction;
	}

	SCOPE_CYCLE_COUNTER(WeightMapPaintToolTick);

	// process the undo update
	if (bUndoUpdatePending)
	{
		// wait for updates
		WaitForPendingUndoRedoUpdate();

		// post rendering update
		DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(AccumulatedTriangleROI, EMeshRenderAttributeFlags::VertexColors);
		GetToolManager()->PostInvalidation();

		// ignore stamp and wait for next tick to do anything else
		bUndoUpdatePending = false;
		return;
	}

	// Get value at brush location
	const bool bShouldPickWeight = bPendingPickWeight && IsStampPending() == false;
	const bool bShouldUpdateValueAtBrush = IsInBrushSubMode();

	if (bShouldPickWeight || bShouldUpdateValueAtBrush)
	{
		if (GetSculptMesh()->IsTriangle(GetBrushTriangleID()))
		{
			if (FilterProperties->ValueAtBrushQueryType == EClothEditorWeightMapPaintQueryType::NearestVertexAccurate)
			{
				UpdateROI(HoverStamp);
			}

			const double HitWeightValue = GetCurrentWeightValueUnderBrush();

			if (bShouldPickWeight)
			{
				FilterProperties->AttributeValue = HitWeightValue;
				NotifyOfPropertyChangeByTool(FilterProperties);
			}

			if (bShouldUpdateValueAtBrush)
			{
				FilterProperties->ValueAtBrush = HitWeightValue;
			}
		}
		bPendingPickWeight = false;
	}

	auto ExecuteStampOperation = [this](int StampIndex, const FRay& StampRay)
		{
			SCOPE_CYCLE_COUNTER(WeightMapPaintTool_Tick_ApplyStampBlock);

			// update sculpt ROI
			UpdateROI(CurrentStamp);

			// append updated ROI to modified region (async)
			FDynamicMesh3* Mesh = GetSculptMesh();
			TFuture<void> AccumulateROI = Async(UE::Chaos::ClothAsset::Private::WeightPaintToolAsyncExecTarget, [&]()
				{
					UE::Geometry::VertexToTriangleOneRing(Mesh, VertexROI, AccumulatedTriangleROI);
				});

			// apply the stamp
			bool bWeightsModified = ApplyStamp();

			if (bWeightsModified)
			{
				SCOPE_CYCLE_COUNTER(WeightMapPaintTool_Tick_UpdateMeshBlock);
				UpdateVertexColorOverlay(&TriangleROI);
				DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TriangleROI, EMeshRenderAttributeFlags::VertexColors);
				GetToolManager()->PostInvalidation();
			}

			// we don't really need to wait for these to happen to end Tick()...
			AccumulateROI.Wait();
		};

	if (IsInBrushSubMode())
	{
		ProcessPerTickStamps(
			[this](const FRay& StampRay) -> bool {
				return UpdateStampPosition(StampRay);
			}, ExecuteStampOperation);
	}

}

bool UClothEditorWeightMapPaintTool::CanAccept() const
{
	return bAnyChangeMade || 
		UpdateWeightMapProperties->Name != WeightMapNodeToUpdate->OutputName.StringValue ||
		UpdateWeightMapProperties->MapOverrideType != WeightMapNodeToUpdate->MapOverrideType;
}

FColor UClothEditorWeightMapPaintTool::GetColorForWeightValue(double WeightValue)
{
	FColor MaxColor = LinearColors::White3b();
	FColor MinColor = LinearColors::Black3b();
	FColor Color;
	double ClampedValue = FMath::Clamp(WeightValue, 0.0, 1.0);
	Color.R = FMath::LerpStable(MinColor.R, MaxColor.R, ClampedValue);
	Color.G = FMath::LerpStable(MinColor.G, MaxColor.G, ClampedValue);
	Color.B = FMath::LerpStable(MinColor.B, MaxColor.B, ClampedValue);
	Color.A = 1.0;

	return Color;
}

void UClothEditorWeightMapPaintTool::FloodFillCurrentWeightAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	BeginChange();

	const float SetWeightValue = FilterProperties->AttributeValue;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 vid : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(vid);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 vid : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[vid]])
			{
				float PreviousValue;
				ActiveWeightMap->GetValue(Idx, &PreviousValue);
				
				ensure(MeshIndexToNodeIndex(Idx) == DynamicMeshToWeight[vid]);
				ActiveChangeBuilder->UpdateValue(DynamicMeshToWeight[vid], PreviousValue, SetWeightValue);

				ActiveWeightMap->SetValue(Idx, &SetWeightValue);
			}
		}
	}
	else
	{
		for (int32 vid : TempROIBuffer)
		{
			float PreviousValue;
			ActiveWeightMap->GetValue(vid, &PreviousValue);

			ensure(MeshIndexToNodeIndex(vid) == vid);
			ActiveChangeBuilder->UpdateValue(vid, PreviousValue, SetWeightValue);

			ActiveWeightMap->SetValue(vid, &SetWeightValue); 
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UClothEditorWeightMapPaintTool::ClearAllWeightsAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}

	BeginChange();

	float SetWeightValue = 0.0f;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 vid : Mesh->VertexIndicesItr())
	{
		TempROIBuffer.Add(vid);
	}

	if (bHaveDynamicMeshToWeightConversion)
	{
		for (int32 vid : TempROIBuffer)
		{
			for (const int32 Idx : WeightToDynamicMesh[DynamicMeshToWeight[vid]])
			{
				float PreviousValue;
				ActiveWeightMap->GetValue(Idx, &PreviousValue);

				ensure(MeshIndexToNodeIndex(Idx) == DynamicMeshToWeight[vid]);
				ActiveChangeBuilder->UpdateValue(DynamicMeshToWeight[vid], PreviousValue, SetWeightValue);

				ActiveWeightMap->SetValue(Idx, &SetWeightValue);
			}
		}
	}
	else
	{
		for (int32 vid : TempROIBuffer)
		{
			float PreviousValue;
			ActiveWeightMap->GetValue(vid, &PreviousValue);

			ensure(MeshIndexToNodeIndex(vid) == vid);
			ActiveChangeBuilder->UpdateValue(vid, PreviousValue, SetWeightValue);

			ActiveWeightMap->SetValue(vid, &SetWeightValue);
		}
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UClothEditorWeightMapPaintTool::InvertWeightsAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}
	BeginChange();

	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	checkf(Mesh, TEXT("Paint Tool's DynamicMeshComponent has no FDynamicMesh"));

	for (const int32 VertexID : Mesh->VertexIndicesItr())
	{
		float PreviousValue;
		ActiveWeightMap->GetValue(VertexID, &PreviousValue);
		const float NewWeightValue = 1.0f - PreviousValue;
		ActiveWeightMap->SetValue(VertexID, &NewWeightValue);

		const int32 NodeIndex = MeshIndexToNodeIndex(VertexID);
		ActiveChangeBuilder->UpdateValue(NodeIndex, PreviousValue, NewWeightValue);
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UClothEditorWeightMapPaintTool::MultiplyWeightsAction()
{
	if (!ActiveWeightMap)
	{
		return;
	}
	BeginChange();

	const float WeightMultiplierValue = FilterProperties->AttributeValue;

	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	checkf(Mesh, TEXT("Paint Tool's DynamicMeshComponent has no FDynamicMesh"));

	for (const int32 VertexID : Mesh->VertexIndicesItr())
	{
		float PreviousValue;
		ActiveWeightMap->GetValue(VertexID, &PreviousValue);
		const float NewWeightValue = FMath::Clamp(WeightMultiplierValue * PreviousValue, 0.f, 1.f);
		ActiveWeightMap->SetValue(VertexID, &NewWeightValue);

		const int32 NodeIndex = MeshIndexToNodeIndex(VertexID);
		ActiveChangeBuilder->UpdateValue(NodeIndex, PreviousValue, NewWeightValue);
	}

	// update colors
	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UClothEditorWeightMapPaintTool::ClearHiddenAction()
{
	HiddenTriangles.Reset();

	for (TPair<int32, bool>& PatternSelection : ShowHideProperties->ShowPatterns)
	{
		PatternSelection.Value = false;
	}

	UpdateVertexColorOverlay();
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);

	MeshElementsDisplay->NotifyMeshChanged();
	DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
	GetToolManager()->PostInvalidation();
}


void UClothEditorWeightMapPaintTool::UpdateSelectedNode()
{
	check(ActiveWeightMap);
	TArray<float> CurrentWeights;
	GetCurrentWeightMap(CurrentWeights);

	checkf(WeightMapNodeToUpdate, TEXT("Expected non-null pointer to Add Weight Map Node"));

	// Save previous state for undo
	if (UDataflow* const Dataflow = DataflowContextObject->GetDataflowAsset())
	{
		GetToolManager()->GetContextTransactionsAPI()->AppendChange(Dataflow, 
			FChaosClothAssetWeightMapNode::MakeWeightMapNodeChange(*WeightMapNodeToUpdate),
			LOCTEXT("WeightMapNodeChangeDescription", "Update Weight Map Node"));
	}


	WeightMapNodeToUpdate->MapOverrideType = UpdateWeightMapProperties->MapOverrideType;
	WeightMapNodeToUpdate->OutputName.StringValue = UpdateWeightMapProperties->Name;

	if (bHaveDynamicMeshToWeightConversion)
	{
		TArray<float> NodeWeights;
		NodeWeights.Init(0.f, WeightToDynamicMesh.Num());
		for (int32 DynamicMeshIdx = 0; DynamicMeshIdx < CurrentWeights.Num(); ++DynamicMeshIdx)
		{
			NodeWeights[DynamicMeshToWeight[DynamicMeshIdx]] = CurrentWeights[DynamicMeshIdx];
		}
		WeightMapNodeToUpdate->SetVertexWeights(InputWeightMap, NodeWeights);
	}
	else
	{
		WeightMapNodeToUpdate->SetVertexWeights(InputWeightMap, CurrentWeights);
	}
	
	WeightMapNodeToUpdate->Invalidate();
}


int32 UClothEditorWeightMapPaintTool::MeshIndexToNodeIndex(int32 MeshVertexIndex) const
{
	if (bHaveDynamicMeshToWeightConversion)
	{
		return DynamicMeshToWeight[MeshVertexIndex];
	}
	else
	{
		return MeshVertexIndex;
	}
}

void UClothEditorWeightMapPaintTool::UpdateMapValuesFromNodeValues(const TArray<int32>& Indices, const TArray<float>& Values)
{
	check(Indices.Num() == Values.Num());

	for (int32 PairIndex = 0; PairIndex < Indices.Num(); ++PairIndex)
	{
		const int32 BufferIndex = Indices[PairIndex];
		const float Value = Values[PairIndex];

		if (bHaveDynamicMeshToWeightConversion)
		{
			for (const int32 MeshIndex : WeightToDynamicMesh[BufferIndex])
			{
				ActiveWeightMap->SetValue(MeshIndex, &Value);
			}
		}
		else
		{
			ActiveWeightMap->SetValue(BufferIndex, &Value);
		}
	}

	if (Indices.Num() > 0)
	{
		UpdateVertexColorOverlay();
		DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	}
}


//
// Change Tracking
//


void UClothEditorWeightMapPaintTool::BeginChange()
{
	check(ActiveChangeBuilder == nullptr);
	ActiveChangeBuilder = MakeUnique<TIndexedValuesChangeBuilder<float, FNodeBufferWeightChange>>();
	ActiveChangeBuilder->BeginNewChange();

	LongTransactions.Open(LOCTEXT("WeightPaintChange", "Weight Stroke"), GetToolManager());
}

void UClothEditorWeightMapPaintTool::EndChange()
{
	check(ActiveChangeBuilder);

	bAnyChangeMade = true;

	TUniquePtr<FNodeBufferWeightChange> EditResult = ActiveChangeBuilder->ExtractResult();
	ActiveChangeBuilder.Reset();

	EditResult->ApplyFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<float>& Values)
	{
		UClothEditorWeightMapPaintTool* const Tool = CastChecked<UClothEditorWeightMapPaintTool>(Object);
		Tool->UpdateMapValuesFromNodeValues(Indices, Values);
	};

	EditResult->RevertFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<float>& Values)
	{
		UClothEditorWeightMapPaintTool* const Tool = CastChecked<UClothEditorWeightMapPaintTool>(Object);
		Tool->UpdateMapValuesFromNodeValues(Indices, Values);
	};

	TUniquePtr<TWrappedToolCommandChange<FNodeBufferWeightChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FNodeBufferWeightChange>>();
	NewChange->WrappedChange = MoveTemp(EditResult);
	NewChange->BeforeModify = [this](bool bRevert)
	{
		this->WaitForPendingUndoRedoUpdate();
	};
	NewChange->AfterModify = [this](bool bRevert)
	{
		this->UpdateVertexColorOverlay();
		this->DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	};

	GetToolManager()->EmitObjectChange(this, MoveTemp(NewChange), LOCTEXT("VertexWeightChange", "Weight Stroke"));

	LongTransactions.Close(GetToolManager());
}


void UClothEditorWeightMapPaintTool::WaitForPendingUndoRedoUpdate()
{
	if (bUndoUpdatePending)
	{
		bUndoUpdatePending = false;
	}
}

void UClothEditorWeightMapPaintTool::OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshVertexChange* Change, bool bRevert)
{
	// update octree
	FDynamicMesh3* Mesh = GetSculptMesh();

	// make sure any previous async computations are done, and update the undo ROI
	if (bUndoUpdatePending)
	{
		// we should never hit this anymore, because of pre-change calling WaitForPendingUndoRedoUpdate()
		WaitForPendingUndoRedoUpdate();

		// this is not right because now we are going to do extra recomputation, but it's very messy otherwise...
		UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}
	else
	{
		AccumulatedTriangleROI.Reset();
		UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}

	// note that we have a pending update
	bUndoUpdatePending = true;
}


void UClothEditorWeightMapPaintTool::PrecomputeFilterData()
{
	const FDynamicMesh3* Mesh = GetSculptMesh();
	
	TriNormals.SetNum(Mesh->MaxTriangleID());
	ParallelFor(Mesh->MaxTriangleID(), [&](int32 tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			TriNormals[tid] = Mesh->GetTriNormal(tid);
		}
	});

	const FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();
	const FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->PrimaryUV();
	UVSeamEdges.SetNum(Mesh->MaxEdgeID());
	NormalSeamEdges.SetNum(Mesh->MaxEdgeID());
	ParallelFor(Mesh->MaxEdgeID(), [&](int32 eid)
	{
		if (Mesh->IsEdge(eid))
		{
			UVSeamEdges[eid] = UVs ? UVs->IsSeamEdge(eid) : false;
			NormalSeamEdges[eid] = Normals ? Normals->IsSeamEdge(eid) : false;
		}
	});
}

double UClothEditorWeightMapPaintTool::GetCurrentWeightValue(int32 VertexId) const
{
	float WeightValue = 0.0;
	if (ActiveWeightMap && VertexId != IndexConstants::InvalidID)
	{
		ActiveWeightMap->GetValue(VertexId, &WeightValue);
	}
	return WeightValue;
}

double UClothEditorWeightMapPaintTool::GetCurrentWeightValueUnderBrush() const
{
	if (!ActiveWeightMap)
	{
		return -1.0;
	}

	float WeightValue = -1.0f;

	switch (FilterProperties->ValueAtBrushQueryType)
	{

		case EClothEditorWeightMapPaintQueryType::Interpolated:
		{
			const int32 Tid = GetBrushTriangleID();
			if (Tid != IndexConstants::InvalidID)
			{
				const FDynamicMesh3* Mesh = GetSculptMesh();
				const FIndex3i Vertices = Mesh->GetTriangle(Tid);
				WeightValue = 0.0f;
				for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
				{
					float VertexWeight;
					ActiveWeightMap->GetValue(Vertices[TriangleVertexIndex], &VertexWeight);
					WeightValue += CurrentBaryCentricCoords[TriangleVertexIndex] * VertexWeight;
				}
			}
		}
		break;

		case EClothEditorWeightMapPaintQueryType::NearestVertexFast:
		{
			const int32 VertexID = GetBrushNearestVertex();
			if (VertexID != IndexConstants::InvalidID)
			{
				ActiveWeightMap->GetValue(VertexID, &WeightValue);
			}
		}
		break;

		case EClothEditorWeightMapPaintQueryType::NearestVertexAccurate:
		{
			const int32 VertexID = GetBrushNearestVertexAccurate();
			if (VertexID != IndexConstants::InvalidID)
			{
				ActiveWeightMap->GetValue(VertexID, &WeightValue);
			}
		}

		break;
	}
	return WeightValue;
}

int32 UClothEditorWeightMapPaintTool::GetBrushNearestVertex() const
{
	int TriangleVertex = 0;

	if (CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Y && CurrentBaryCentricCoords.X >= CurrentBaryCentricCoords.Z)
	{
		TriangleVertex = 0;
	}
	else
	{
		if (CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.X && CurrentBaryCentricCoords.Y >= CurrentBaryCentricCoords.Z)
		{
			TriangleVertex = 1;
		}
		else
		{
			TriangleVertex = 2;
		}
	}
	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 Tid = GetBrushTriangleID();
	if (Tid == IndexConstants::InvalidID)
	{
		return IndexConstants::InvalidID;
	}
	
	FIndex3i Vertices = Mesh->GetTriangle(Tid);
	return Vertices[TriangleVertex];
}

int32 UClothEditorWeightMapPaintTool::GetBrushNearestVertexAccurate() const
{
	int32 NearestVertexIndex = INDEX_NONE;
	const int32 Tid = GetBrushTriangleID();

	if (Tid != IndexConstants::InvalidID)
	{
		const FDynamicMesh3* const Mesh = GetSculptMesh();

		FVector3d PointOnSurface(0, 0, 0);
		const FIndex3i Vertices = Mesh->GetTriangle(Tid);
		for (int32 TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
		{
			PointOnSurface += CurrentBaryCentricCoords[TriangleVertexIndex] * Mesh->GetVertex(Vertices[TriangleVertexIndex]);
		}

		double MinDist = UE_BIG_NUMBER;
		for (const int32 VertexIndex : VertexROI)
		{
			const FVector3d& VertexPosition = Mesh->GetVertex(VertexIndex);
			const double CurrDist = FVector3d::Distance(VertexPosition, PointOnSurface);
			if (CurrDist < MinDist)
			{
				MinDist = CurrDist;
				NearestVertexIndex = VertexIndex;
			}
		}
	}

	return NearestVertexIndex;
}


void UClothEditorWeightMapPaintTool::GetCurrentWeightMap(TArray<float>& OutWeights) const
{
	if (ActiveWeightMap)
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();
		const int32 NumVertices = Mesh->VertexCount();
		OutWeights.SetNumUninitialized(NumVertices);
		for (int32 VertexID = 0; VertexID < NumVertices; ++VertexID)
		{
			ActiveWeightMap->GetValue(VertexID, &OutWeights[VertexID]);
		}
	}
}



void UClothEditorWeightMapPaintTool::UpdateSubToolType(EClothEditorWeightMapPaintInteractionType NewType)
{
	// Currenly we mirror base-brush properties in UClothEditorWeightMapPaintBrushFilterProperties, so we never
	// want to show both
	//bool bSculptPropsVisible = (NewType == EClothEditorWeightMapPaintInteractionType::Brush);
	//SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, bSculptPropsVisible);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, false);

	SetToolPropertySourceEnabled(FilterProperties, true);
	SetBrushOpPropsVisibility(false);

	if (NewType != EClothEditorWeightMapPaintInteractionType::Gradient)
	{
		LowValueGradientVertexSelection.Clear();
		HighValueGradientVertexSelection.Clear();
	}
}


void UClothEditorWeightMapPaintTool::UpdateBrushType(EClothEditorWeightMapPaintBrushType BrushType)
{
	FText BaseMessage;
	if (BrushType == EClothEditorWeightMapPaintBrushType::Paint)
	{
		BaseMessage = LOCTEXT("OnStartPaintMode", "Hold Shift to Erase. Use [/] and S/D keys to change brush size (+Shift to small-step). W/E to change Value (+Shift to small-step). Shift-G to get current Value under cursor. Q/A to cycle through brush modes.");
	}
	else if (BrushType == EClothEditorWeightMapPaintBrushType::Smooth)
	{
		BaseMessage = LOCTEXT("OnStartBrushMode", "Hold Shift to Erase. Use [/] and S/D keys to change brush size (+Shift to small-step). Q/A to cycle through brush modes.");
	}

	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetActivePrimaryBrushType((int32)BrushType);

	SetToolPropertySourceEnabled(GizmoProperties, false);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}




void UClothEditorWeightMapPaintTool::RequestAction(EClothEditorWeightMapPaintToolActions ActionType)
{
	if (!bHavePendingAction)
	{
		PendingAction = ActionType;
		bHavePendingAction = true;
	}
}


void UClothEditorWeightMapPaintTool::ApplyAction(EClothEditorWeightMapPaintToolActions ActionType)
{
	switch (ActionType)
	{
	case EClothEditorWeightMapPaintToolActions::FloodFillCurrent:
		FloodFillCurrentWeightAction();
		break;

	case EClothEditorWeightMapPaintToolActions::ClearAll:
		ClearAllWeightsAction();
		break;

	case EClothEditorWeightMapPaintToolActions::Invert:
		InvertWeightsAction();
		break;

	case EClothEditorWeightMapPaintToolActions::Multiply:
		MultiplyWeightsAction();
		break;

	case EClothEditorWeightMapPaintToolActions::ClearHiddenTriangles:
		ClearHiddenAction();
		break;
	}
}


void UClothEditorWeightMapPaintTool::UpdateVertexColorOverlay(const TSet<int>* TrianglesToUpdate)
{
	FDynamicMesh3* const Mesh = GetSculptMesh();
	check(Mesh->HasAttributes());
	check(Mesh->Attributes()->PrimaryColors());
	check(ActiveWeightMap);

	FDynamicMeshColorOverlay* const ColorOverlay = Mesh->Attributes()->PrimaryColors();

	auto SetColorsFromWeights = [&](int TriangleID)
	{
		const FIndex3i Tri = Mesh->GetTriangle(TriangleID);
		const FIndex3i ColorElementTri = ColorOverlay->GetTriangle(TriangleID);

		for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
		{
			float VertexWeight;
			ActiveWeightMap->GetValue(Tri[TriVertIndex], &VertexWeight);
			VertexWeight = FMath::Clamp(VertexWeight, 0.0f, 1.0f);

			FVector4f NewColor;
			if (FilterProperties->bHighlightZeroAndOne && VertexWeight == 0.0f)
			{
				NewColor = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
			} 
			else if (FilterProperties->bHighlightZeroAndOne && VertexWeight == 1.0f)
			{
				NewColor = FVector4f(1.0f, 1.0f, 0.0f, 1.0f);
			}
			else if (FilterProperties->ColorMap == EClothEditorWeightMapDisplayType::BlackAndWhite)
			{
				NewColor = FVector4f(VertexWeight, VertexWeight, VertexWeight, 1.0f);
			}
			else
			{
				NewColor = VertexWeight * FVector4f(0.9f, 0.05f, 0.05f, 1.0f) + (1.0f - VertexWeight) * FVector4f(0.65f, 0.65f, 0.65f, 1.0f);
			}

			ColorOverlay->SetElement(ColorElementTri[TriVertIndex], NewColor);
		}
	};

	if (TrianglesToUpdate)
	{
		for (const int TriangleID : *TrianglesToUpdate)
		{
			SetColorsFromWeights(TriangleID);
		}
	}
	else
	{
		for (const int TriangleID : Mesh->TriangleIndicesItr())
		{
			SetColorsFromWeights(TriangleID);
		}
	}
}


#undef LOCTEXT_NAMESPACE
