// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "PropertySets/WeightMapSetProperties.h"
#include "Mechanics/PolyLassoMarqueeMechanic.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "TransformTypes.h"
#include "ToolDataVisualizer.h"
#include "GroupTopology.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Changes/IndexedAttributeChange.h"

#include "ClothWeightMapPaintTool.generated.h"

#define UE_API CHAOSCLOTHASSETEDITORTOOLS_API

class UMeshElementsVisualizer;
class UWeightMapEraseBrushOpProps;
class UWeightMapPaintBrushOpProps;
class UWeightMapSmoothBrushOpProps;
class UDataflowContextObject;
class UPolygonSelectionMechanic;
struct FChaosClothAssetWeightMapNode;
enum class EChaosClothAssetWeightMapOverrideType : uint8;

DECLARE_STATS_GROUP(TEXT("WeightMapPaintTool"), STATGROUP_WeightMapPaintTool, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_UpdateROI"), WeightMapPaintTool_UpdateROI, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_ApplyStamp"), WeightMapPaintToolApplyStamp, STATGROUP_WeightMapPaintTool );
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick"), WeightMapPaintToolTick, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStampBlock"), WeightMapPaintTool_Tick_ApplyStampBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStamp_Remove"), WeightMapPaintTool_Tick_ApplyStamp_Remove, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_ApplyStamp_Insert"), WeightMapPaintTool_Tick_ApplyStamp_Insert, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_NormalsBlock"), WeightMapPaintTool_Tick_NormalsBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_UpdateMeshBlock"), WeightMapPaintTool_Tick_UpdateMeshBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Tick_UpdateTargetBlock"), WeightMapPaintTool_Tick_UpdateTargetBlock, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Normals_Collect"), WeightMapPaintTool_Normals_Collect, STATGROUP_WeightMapPaintTool);
DECLARE_CYCLE_STAT(TEXT("WeightMapPaintTool_Normals_Compute"), WeightMapPaintTool_Normals_Compute, STATGROUP_WeightMapPaintTool);


UENUM()
enum class EClothEditorWeightMapDisplayType : uint8
{
	BlackAndWhite,
	WhiteAndRed,
	LastValue UMETA(Hidden)
};

/** Mesh Sculpting Brush Types */
UENUM()
enum class EClothEditorWeightMapPaintInteractionType : uint8
{
	Brush,
	Fill,
	PolyLasso,
	Gradient,
	HideTriangles,
	LastValue UMETA(Hidden)
};







/** Mesh Sculpting Brush Types */
UENUM()
enum class EClothEditorWeightMapPaintBrushType : uint8
{
	/** Paint weights */
	Paint UMETA(DisplayName = "Paint"),

	/** Smooth existing weights */
	Smooth UMETA(DisplayName = "Smooth"),

	/** Erase weights */
	Erase UMETA(Hidden, DisplayName = "Erase"),

	LastValue UMETA(Hidden)
};


/** Mesh Sculpting Brush Area Types */
UENUM()
enum class EClothEditorWeightMapPaintBrushAreaType : uint8
{
	Connected,
	Volumetric
};

/** Mesh Sculpting Brush Types */
UENUM()
enum class EClothEditorWeightMapPaintVisibilityType : uint8
{
	None,
	Unoccluded
};

/** Value Query mode  */
UENUM()
enum class EClothEditorWeightMapPaintQueryType : uint8
{
	/** Value is interpolated from triangle vertices */
	Interpolated UMETA(DisplayName = "Interpolated"),

	/** Return the value at the closest vertex of the triangle under the mouse cursor */
	NearestVertexFast UMETA(DisplayName = "Nearest Vertex (Fast)"),

	/** Return the value of the nearest vertex inside the brush radius, even if the vertex is not on the triangle under the mouse cursor */
	NearestVertexAccurate UMETA(DisplayName = "Nearest Vertex (Accurate)")
};


// TODO: Look at EditConditions for all these properties. Which ones make sense for which SubToolType?

UCLASS(MinimalAPI)
class UClothEditorWeightMapPaintBrushFilterProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Display)
	EClothEditorWeightMapDisplayType ColorMap = EClothEditorWeightMapDisplayType::BlackAndWhite;

	UPROPERTY(EditAnywhere, Category = Display)
	bool bHighlightZeroAndOne = false;

	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Action"))
	EClothEditorWeightMapPaintInteractionType SubToolType = EClothEditorWeightMapPaintInteractionType::Brush;

	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Brush Mode", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush"))
	EClothEditorWeightMapPaintBrushType PrimaryBrushType = EClothEditorWeightMapPaintBrushType::Paint;

	/** Relative size of brush */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Brush Size", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush"))
	float BrushSize = 0.25f;

	/** Relative size of falloff region inside the brush */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush && PrimaryBrushType == EClothEditorWeightMapPaintBrushType::Smooth"))
	float Falloff = 0.5;

	/** Allow the Brush to hit the back-side of the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush"))
	bool bHitBackFaces = true;

	/** The new value to paint on the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"(SubToolType == EClothEditorWeightMapPaintInteractionType::Brush && PrimaryBrushType == EClothEditorWeightMapPaintBrushType::Paint) || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill || SubToolType == EClothEditorWeightMapPaintInteractionType::PolyLasso"))
	double AttributeValue = 1;

	/** How quickly each brush stroke will drive mesh values towards the desired value */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	double Strength = 1.0;

	/** The Gradient lower limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Gradient"))
	double GradientLowValue = 0.0;

	/** The Gradient upper limit value */
	UPROPERTY(EditAnywhere, Category = Gradient, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1,
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Gradient"))
	double GradientHighValue = 1.0;


	/** The Region affected by the current operation will be bounded by edge angles larger than this threshold */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (UIMin = "0.0", ClampMin = "0.0", UIMax = "180.0", ClampMax = "180.0",
		HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	float AngleThreshold = 180.0f;

	/** The Region affected by the current operation will be bounded by UV borders/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	bool bUVSeams = false;

	/** The Region affected by the current operation will be bounded by Hard Normal edges/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = 
		"SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	bool bNormalSeams = false;

	/** Control which triangles can be affected by the current operation based on visibility. Applied after all other filters. */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (HideEditConditionToggle, EditConditionHides, EditCondition =
		"SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill || SubToolType == EClothEditorWeightMapPaintInteractionType::PolyLasso"))
	EClothEditorWeightMapPaintVisibilityType VisibilityFilter = EClothEditorWeightMapPaintVisibilityType::None;

	/** The weight value at the brush indicator */
	UPROPERTY(VisibleAnywhere, Transient, Category = Query, meta = (NoResetToDefault, 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	double ValueAtBrush = 0;

	/** How to compute the value under brush indicator */
	UPROPERTY(EditAnywhere, Transient, Category = Query, meta = (HideEditConditionToggle, EditConditionHides, 
		EditCondition = "SubToolType == EClothEditorWeightMapPaintInteractionType::Brush || SubToolType == EClothEditorWeightMapPaintInteractionType::Fill"))
	EClothEditorWeightMapPaintQueryType ValueAtBrushQueryType = EClothEditorWeightMapPaintQueryType::NearestVertexFast;

};





UENUM()
enum class EClothEditorWeightMapPaintToolActions
{
	NoAction,

	FloodFillCurrent,
	ClearAll,
	Invert,
	Multiply,
	ClearHiddenTriangles,
};


UCLASS(MinimalAPI)
class UClothEditorMeshWeightMapPaintToolActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	TWeakObjectPtr<UClothEditorWeightMapPaintTool> ParentTool;

	void Initialize(UClothEditorWeightMapPaintTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EClothEditorWeightMapPaintToolActions Action);

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 10))
	void ClearAll()
	{
		PostAction(EClothEditorWeightMapPaintToolActions::ClearAll);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 12))
	void FloodFillCurrent()
	{
		PostAction(EClothEditorWeightMapPaintToolActions::FloodFillCurrent);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 13))
	void Invert()
	{
		PostAction(EClothEditorWeightMapPaintToolActions::Invert);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 14))
	void Multiply()
	{
		PostAction(EClothEditorWeightMapPaintToolActions::Multiply);
	}

};

UCLASS(MinimalAPI)
class UClothEditorUpdateWeightMapProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = UpdateNode, meta = (DisplayName = "Name"))
	FString Name;

	UPROPERTY(EditAnywhere, Category = UpdateNode)
	EChaosClothAssetWeightMapOverrideType MapOverrideType;

private:

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
};

/// Property panel for controlling hiding triangles
UCLASS(MinimalAPI)
class UClothEditorMeshWeightMapPaintToolShowHideProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	TWeakObjectPtr<UClothEditorWeightMapPaintTool> ParentTool;
	void Initialize(UClothEditorWeightMapPaintTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EClothEditorWeightMapPaintToolActions Action);

	/** Toggles whether each pattern is shown or hidden */
	UPROPERTY(EditAnywhere, EditFixedSize, Transient, Category = TriangleVisibility, meta = (DisplayPriority = 2));
	TMap<int32, bool> ShowPatterns;

	/** Unhide all triangles */
	UFUNCTION(CallInEditor, Category = TriangleVisibility, meta = (DisplayPriority = 1))
	void ShowAll()
	{
		PostAction(EClothEditorWeightMapPaintToolActions::ClearHiddenTriangles);
	}
};


/**
 * Mesh Element Paint Tool Class
 */
UCLASS(MinimalAPI)
class UClothEditorWeightMapPaintTool : public UMeshSculptToolBase
{
	GENERATED_BODY()

public:
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API bool IsInBrushSubMode() const;

	UE_API virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology) override;

	UE_API void SetDataflowContextObject(TObjectPtr<UDataflowContextObject> InDataflowContextObject);

	virtual void SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior) override;

private:

	// Initialization support
	UE_API void InitializeSculptMeshFromTarget();
	UE_API void UpdateShowHideProperties();

	// Make sure things are set up correctly after initialization or re-initialization
	UE_API void PostSetupCheck() const;

public:


	/** Filters on paint brush */
	UPROPERTY()
	TObjectPtr<UClothEditorWeightMapPaintBrushFilterProperties> FilterProperties;


private:
	UPROPERTY()
	TObjectPtr<UWeightMapPaintBrushOpProps> PaintBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UWeightMapSmoothBrushOpProps> SmoothBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UWeightMapEraseBrushOpProps> EraseBrushOpProperties;

	UE_API void FloodFillCurrentWeightAction();
	UE_API void ClearAllWeightsAction();
	UE_API void InvertWeightsAction();
	UE_API void MultiplyWeightsAction();
	UE_API void ClearHiddenAction();

public:
	UE_API void SetVerticesToWeightMap(const TSet<int32>& Vertices, double WeightValue, bool bIsErase);

	UE_API bool HaveVisibilityFilter() const;
	UE_API void ApplyVisibilityFilter(const TArray<int32>& Vertices, TArray<int32>& VisibleVertices);
	UE_API void ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer);

	// we override these so we can update the separate BrushSize property added for this tool
	UE_API virtual void IncreaseBrushRadiusAction();
	UE_API virtual void DecreaseBrushRadiusAction();
	UE_API virtual void IncreaseBrushRadiusSmallStepAction();
	UE_API virtual void DecreaseBrushRadiusSmallStepAction();

protected:
	// UMeshSculptToolBase API
	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() { return DynamicMeshComponent; }
	virtual FDynamicMesh3* GetBaseMesh() { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const { check(false); return nullptr; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual void UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay) override;


	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnCancelStroke() override;
	UE_API virtual void OnEndStroke() override;

	UE_API virtual TUniquePtr<FMeshSculptBrushOp>& GetActiveBrushOp();

	virtual bool SharesBrushPropertiesChanges() const override { return false; }

	UE_API virtual void InitializeBrushSizeRange(const UE::Geometry::FAxisAlignedBox3d& TargetBounds) override;

	UE_API virtual void NextBrushModeAction() override;
	UE_API virtual void PreviousBrushModeAction() override;

	// Note: these will actually modify the brush's Attribute Value since we don't use Speed in our brush
	UE_API virtual void IncreaseBrushSpeedAction() override;
	UE_API virtual void DecreaseBrushSpeedAction() override;


	// end UMeshSculptToolBase API



	//
	// Action support
	//

public:
	UE_API virtual void RequestAction(EClothEditorWeightMapPaintToolActions ActionType);
	
	UPROPERTY()
	TObjectPtr<UClothEditorMeshWeightMapPaintToolActions> ActionsProps;

	UPROPERTY()
	TObjectPtr<UClothEditorUpdateWeightMapProperties> UpdateWeightMapProperties;

protected:
	bool bHavePendingAction = false;
	EClothEditorWeightMapPaintToolActions PendingAction;
	UE_API virtual void ApplyAction(EClothEditorWeightMapPaintToolActions ActionType);



	//
	// Marquee Support
	//
public:
	UPROPERTY()
	TObjectPtr<UPolyLassoMarqueeMechanic> PolyLassoMechanic;

protected:
	UE_API void OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled);


	// 
	// Gradient Support
	//
	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> PolygonSelectionMechanic;

	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial = nullptr;

	TUniquePtr<UE::Geometry::FTriangleGroupTopology> GradientSelectionTopology = nullptr;

	FToolDataVisualizer GradientSelectionRenderer;

	UE::Geometry::FGroupTopologySelection LowValueGradientVertexSelection;
	UE::Geometry::FGroupTopologySelection HighValueGradientVertexSelection;

	UE_API void ComputeGradient();
	UE_API void OnSelectionModified();


	//
	// Show/Hide support
	//
protected:
	UPROPERTY()
	TObjectPtr<UClothEditorMeshWeightMapPaintToolShowHideProperties> ShowHideProperties;

	// Hidden triangles are not rendered, their wireframes are not rendered, and they don't block ray casts from the mouse
	TSet<int32> HiddenTriangles;

	// Pending hidden triangles are triangles that are selected by the current HideTriangles brush stroke. When the stroke finishes, these pending triangles are
	// added to the set of hidden triangles, and this set is cleared.
	// 
	// Pending hidden triangles are not rendered, but their wireframes *are* rendered, and they *do* block ray casts.
	// - Wireframe update is delayed because its a relatively expensive operation, so we do it only when the stroke finishes
	// - Ray cast blocking is delayed because otherwise one small mouse drag would drill through multiple mesh layers
	TSet<int32> PendingHiddenTriangles;

	// Triangle index offset and number of triangles for each non-empty pattern. This is used when we want to hide/show entire patterns.
	TArray<TPair<int32, int32>> PatternTriangleOffsetAndNum;

	//
	// Internals
	//

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

	UPROPERTY()
	TObjectPtr<UDataflowContextObject> DataflowContextObject = nullptr;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshVertexChange* Change, bool bRevert);
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	TConstArrayView<float> InputWeightMap;

	UE::Geometry::FDynamicMeshWeightAttribute* ActiveWeightMap;
	UE_API double GetCurrentWeightValue(int32 VertexId) const;
	UE_API double GetCurrentWeightValueUnderBrush() const;
	FVector3d CurrentBaryCentricCoords;
	UE_API int32 GetBrushNearestVertex() const;
	UE_API int32 GetBrushNearestVertexAccurate() const;

	UE_API void GetCurrentWeightMap(TArray<float>& OutWeights) const;

	UE_API void UpdateSubToolType(EClothEditorWeightMapPaintInteractionType NewType);

	UE_API void UpdateBrushType(EClothEditorWeightMapPaintBrushType BrushType);

	TSet<int32> AccumulatedTriangleROI;
	bool bUndoUpdatePending = false;
	TArray<int> NormalsBuffer;
	UE_API void WaitForPendingUndoRedoUpdate();

	TArray<int> TempROIBuffer;
	TArray<int> VertexROI;
	TArray<bool> VisibilityFilterBuffer;
	TSet<int> VertexSetBuffer;
	TSet<int> TriangleROI;
	UE_API void UpdateROI(const FSculptBrushStamp& CurrentStamp);

	EClothEditorWeightMapPaintBrushType PendingStampType = EClothEditorWeightMapPaintBrushType::Paint;

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API bool ApplyStamp();

	UE::Geometry::FDynamicMeshOctree3 Octree;

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	bool GetInEraseStroke()
	{
		// Re-use the smoothing stroke key (shift) for erase stroke in the weight paint tool
		return GetInSmoothingStroke();
	}


	bool bPendingPickWeight = false;

	TArray<int32> ROITriangleBuffer;
	TArray<double> ROIWeightValueBuffer;
	UE_API bool SyncMeshWithWeightBuffer(FDynamicMesh3* Mesh);
	UE_API bool SyncWeightBufferWithMesh(const FDynamicMesh3* Mesh);


	// Undo/Redo change support

	class FNodeBufferWeightChange : public TCustomIndexedValuesChange<float, int32>
	{
	public:
		virtual FString ToString() const override
		{
			return FString(TEXT("Cloth Vertex Scalar Weight Change"));
		}
	};
	TUniquePtr<TIndexedValuesChangeBuilder<float, FNodeBufferWeightChange>> ActiveChangeBuilder;
	UE_API void BeginChange();
	UE_API void EndChange();

	// The corresponding FChaosClothAssetWeightMapNode has a buffer of scalar weights. Depending on bHaveDynamicMeshToWeightConversion, the 
	// indexing of this buffer might be different than the mesh vertex indexing. This function returns the corresponding index in the node buffer for
	// a given mesh vertex index.
	UE_API int32 MeshIndexToNodeIndex(int32 MeshVertexIndex) const;

	// Update the active weight map attribute from the map of indices/values. The inputs are given in "node buffer format" (see function above).
	UE_API void UpdateMapValuesFromNodeValues(const TArray<int32>& Indices, const TArray<float>& Values);


	UE_API FColor GetColorForWeightValue(double WeightValue);

	TArray<FVector3d> TriNormals;
	TArray<int32> UVSeamEdges;
	TArray<int32> NormalSeamEdges;
	UE_API void PrecomputeFilterData();

	// DynamicMesh might be unwelded mesh, but weights are on the welded mesh.
	bool bHaveDynamicMeshToWeightConversion = false;
	TArray<int32> DynamicMeshToWeight; 
	TArray<TArray<int32>> WeightToDynamicMesh;

protected:
	virtual bool ShowWorkPlane() const override { return false; }

	friend class UClothEditorWeightMapPaintToolBuilder;
	UE_API void NotifyTargetChanged();

	bool bAnyChangeMade = false;

	// Node graph editor support

	FChaosClothAssetWeightMapNode* WeightMapNodeToUpdate = nullptr;

	UE_API void UpdateSelectedNode();

	UE_API void UpdateVertexColorOverlay(const TSet<int>* TrianglesToUpdate = nullptr);

};



#undef UE_API
