// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Dataflow/DataflowEdNode.h"
#include "Dataflow/DataflowGraphEditor.h"
#include "Dataflow/DataflowObject.h"
#include "ClothEditorContextObject.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UDataflow;
class UEdGraphNode;

namespace UE::Chaos::ClothAsset
{
enum class EClothPatternVertexType : uint8;
}
struct FManagedArrayCollection;


UCLASS()
class UE_DEPRECATED(5.6, "Use UDataflowContextObject instead") CHAOSCLOTHASSETEDITORTOOLS_API UClothEditorContextObject : public UObject
{
	GENERATED_BODY()

public:
	void Init(TWeakPtr<SDataflowGraphEditor> DataflowGraphEditor, TWeakPtr<UE::Dataflow::FEngineContext> DataflowContext, UE::Chaos::ClothAsset::EClothPatternVertexType InConstructionViewMode, TWeakPtr<FManagedArrayCollection> SelectedClothCollection, bool bInUsingInputCollection);

	UE_DEPRECATED(5.5, "Use the Init with a DataflowContext")
	void Init(TWeakPtr<SDataflowGraphEditor> InDataflowGraphEditor, UE::Chaos::ClothAsset::EClothPatternVertexType InConstructionViewMode, TWeakPtr<FManagedArrayCollection> InSelectedClothCollection, TWeakPtr<FManagedArrayCollection> InSelectedInputClothCollection = nullptr)
	{
		Init(InDataflowGraphEditor, nullptr, InConstructionViewMode, InSelectedClothCollection, false);
	}

	/**
	* Get a single selected node of the specified type. Return nullptr if the specified node is not selected, or if multiple nodes are selected
	*/
	template<typename NodeType>
	NodeType* GetSingleSelectedNodeOfType() const
	{
		const TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin();

		if (UEdGraphNode* const SingleSelectedNode = GraphEditor->GetSingleSelectedNode())
		{
			UDataflowEdNode* const SelectedDataflowEdNode = CastChecked<UDataflowEdNode>(SingleSelectedNode);
			if (const TSharedPtr<FDataflowNode> DataflowNode = SelectedDataflowEdNode->GetDataflowNode())	// If the user deletes a node from the graph, the UDataflowEdNode might outlast the FDataflowNode
			{
				if (NodeType* const NodeTypeNode = DataflowNode->AsType<NodeType>())
				{
					return NodeTypeNode;
				}
			}
		}

		return nullptr;
	}
	
	TWeakPtr<UE::Dataflow::FEngineContext> GetDataflowContext() const 
	{
		return DataflowContext;
	}

	void SetDataflowContext(TWeakPtr<UE::Dataflow::FEngineContext> InDataflowContext)
	{
		DataflowContext = InDataflowContext;
	}

	UDataflow* GetDataflowAsset() const
	{
		if (const TSharedPtr<SDataflowGraphEditor> GraphEditor = DataflowGraphEditor.Pin())
		{
			if (UEdGraph* const EdGraph = GraphEditor->GetCurrentGraph())
			{
				if (UDataflow* const Dataflow = Cast<UDataflow>(EdGraph))
				{
					return Dataflow;
				}
			}
		}
		return nullptr;
	}

	void SetClothCollection(UE::Chaos::ClothAsset::EClothPatternVertexType ViewMode, TWeakPtr<FManagedArrayCollection> ClothCollection, bool bInUsingInputCollection);

	UE_DEPRECATED(5.5, "SetClothCollection no longer takes a separate InputClothCollection argument")
	void SetClothCollection(UE::Chaos::ClothAsset::EClothPatternVertexType ViewMode, TWeakPtr<FManagedArrayCollection> ClothCollection, TWeakPtr<FManagedArrayCollection> InputClothCollection = nullptr)
	{
		SetClothCollection(ViewMode, ClothCollection, false);
	}

	const TWeakPtr<const FManagedArrayCollection> GetSelectedClothCollection() const { return SelectedClothCollection; }
	UE_DEPRECATED(5.5, "There is no longer a separate input cloth collection.")
	const TWeakPtr<const FManagedArrayCollection> GetSelectedInputClothCollection() const { return SelectedClothCollection; }
	UE::Chaos::ClothAsset::EClothPatternVertexType GetConstructionViewMode() const { return ConstructionViewMode; }
	bool IsUsingInputCollection() const { return bUsingInputCollection; }
private:

	TWeakPtr<SDataflowGraphEditor> DataflowGraphEditor;
	TWeakPtr<UE::Dataflow::FEngineContext> DataflowContext;

	UE::Chaos::ClothAsset::EClothPatternVertexType ConstructionViewMode;
	TWeakPtr<const FManagedArrayCollection> SelectedClothCollection;
	bool bUsingInputCollection = false;
};


