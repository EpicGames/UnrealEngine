// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/Interface.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "DataflowEditorTools/DataflowEditorToolBuilder.h"

#include "ClothEditorToolBuilders.generated.h"

#define UE_API CHAOSCLOTHASSETEDITORTOOLS_API

class UDataflowContextObject;

namespace UE::Dataflow
{
	class IDataflowConstructionViewMode;
}

namespace UE::Chaos::ClothAsset
{
	enum class EClothPatternVertexType : uint8;
}


UINTERFACE(MinimalAPI)
class UChaosClothAssetEditorToolBuilder : public UInterface
{
	GENERATED_BODY()
};


class IChaosClothAssetEditorToolBuilder
{
	GENERATED_BODY()

public:

	UE_DEPRECATED(5.5, "Please use the version taking ContextObject")
	virtual void GetSupportedViewModes(TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const {};

	/** Returns all Construction View modes that this tool can operate in. The first element should be the preferred mode to switch to if necessary. */
	virtual void GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const = 0;

	/** Returns whether or not view can be set to wireframe when this tool is active.. */
	virtual bool CanSetConstructionViewWireframeActive() const { return true; }
};


UCLASS(MinimalAPI)
class UClothEditorWeightMapPaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder, public IChaosClothAssetEditorToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:

	// IDataflowEditorToolBuilder
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual void SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) override;

	// IChaosClothAssetEditorToolBuilder
	UE_API virtual void GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const override;
	virtual bool CanSetConstructionViewWireframeActive() const { return false; }

	// UMeshSurfacePointMeshEditingToolBuilder
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


UCLASS(MinimalAPI)
class UClothMeshSelectionToolBuilder : public UInteractiveToolWithToolTargetsBuilder, public IChaosClothAssetEditorToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

	virtual void GetSupportedViewModes(TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const override {};

private:

	// IDataflowEditorToolBuilder
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual void SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) override;

	// IChaosClothAssetEditorToolBuilder
	UE_API virtual void GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const override;
	virtual bool CanSetConstructionViewWireframeActive() const { return false; }

	// UInteractiveToolWithToolTargetsBuilder
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

UCLASS(MinimalAPI)
class UClothTransferSkinWeightsToolBuilder : public USingleSelectionMeshEditingToolBuilder, public IChaosClothAssetEditorToolBuilder, public IDataflowEditorToolBuilder
{
	GENERATED_BODY()

private:

	// IDataflowEditorToolBuilder
	UE_API virtual void GetSupportedConstructionViewModes(const UDataflowContextObject& ContextObject, TArray<const UE::Dataflow::IDataflowConstructionViewMode*>& Modes) const override;
	UE_API virtual bool CanSceneStateChange(const UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual void SceneStateChanged(UInteractiveTool* ActiveTool, const FToolBuilderState& SceneState) override;

	// IChaosClothAssetEditorToolBuilder
	UE_API virtual void GetSupportedViewModes(const UDataflowContextObject& ContextObject, TArray<UE::Chaos::ClothAsset::EClothPatternVertexType>& Modes) const override;

	// USingleSelectionMeshEditingToolBuilder
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


namespace UE::Chaos::ClothAsset
{
	// Provide a list of Tool default objects for us in TInteractiveToolCommands::RegisterCommands()
	void CHAOSCLOTHASSETEDITORTOOLS_API GetClothEditorToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs);

	// Mapping from Dataflow View Mode to Cloth View Mode. Input object must be one of FCloth2DSimViewMode, FCloth3DSimViewMode, or FClothRenderViewMode, defined in ClothDataflowViewModes.h
	EClothPatternVertexType CHAOSCLOTHASSETEDITORTOOLS_API DataflowViewModeToClothViewMode(const UE::Dataflow::IDataflowConstructionViewMode* DataflowViewMode);

	// Mapping from Cloth View Mode to Dataflow View Mode name. Ouptut will be one of "Cloth2DSimView", "Cloth3DSimView", or "ClothRenderView", as defined in ClothDataflowViewModes.cpp
	FName CHAOSCLOTHASSETEDITORTOOLS_API ClothViewModeToDataflowViewModeName(EClothPatternVertexType ClothViewMode);
}

#undef UE_API
